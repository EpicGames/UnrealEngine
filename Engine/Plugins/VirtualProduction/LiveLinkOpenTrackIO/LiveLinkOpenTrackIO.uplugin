{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkOpenTrackIO",
	"Description": "Live Link plugin for supporting OpenTrackIO (https://opentrackio.org) devices in Unreal Engine or Live Link Hub.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules": [
		{
			"Name": "LiveLinkOpenTrackIO",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"ProgramAllowList": ["LiveLinkHub"]
		}
	],
	"Plugins": [
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "LiveLinkCamera",
			"Enabled": true
		},
		{
			"Name": "LiveLinkLens",
			"Enabled": true
		}
	]

}
