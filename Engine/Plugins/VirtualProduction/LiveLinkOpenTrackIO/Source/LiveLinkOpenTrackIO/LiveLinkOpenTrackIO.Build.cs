// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkOpenTrackIO : ModuleRules
{
	public LiveLinkOpenTrackIO(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"LiveLinkInterface",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Cbor",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
				"LiveLinkLens",
				"Projects",
				"Networking",
				"Sockets",
				"Slate",
				"SlateCore",
				"Serialization"
			}
			);
	}
}
