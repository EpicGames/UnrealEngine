// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenTrackIO.h"

#define LOCTEXT_NAMESPACE "LiveLinkOpenTrackIOModule"

DEFINE_LOG_CATEGORY(LogLiveLinkOpenTrackIO);

void FLiveLinkOpenTrackIOModule::StartupModule()
{
}

void FLiveLinkOpenTrackIOModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLiveLinkOpenTrackIOModule, LiveLinkOpenTrackIO)