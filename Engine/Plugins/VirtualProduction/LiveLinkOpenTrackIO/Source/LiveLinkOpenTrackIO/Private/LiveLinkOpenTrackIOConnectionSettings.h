// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"

#include "LiveLinkOpenTrackIOConnectionSettings.generated.h"


// Pick between Multicast or Unicast for this connection
UENUM(BlueprintType)
enum class ELiveLinkOpenTrackIONetworkProtocol : uint8
{
	Multicast UMETA(DisplayName = "Multicast"),
	Unicast   UMETA(DisplayName = "Unicast")
};


USTRUCT()
struct LIVELINKOPENTRACKIO_API FLiveLinkOpenTrackIOConnectionSettings
{
	GENERATED_BODY()

	/** Using this is equivalent to leaving the SubjectName empty. */
	static constexpr TCHAR AutoSubjectName[] = TEXT("Auto");

	/** If empty or "Auto", one will be automatically generated. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FString SubjectName = AutoSubjectName;

	/** Protocol selection: Multicast (default) or Unicast. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	ELiveLinkOpenTrackIONetworkProtocol Protocol = ELiveLinkOpenTrackIONetworkProtocol::Multicast;

	/** Multicast Group. The last octet is the SourceNumber and should stay within the range [1,200], per the OpenTrackIO spec. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (
		EditCondition = "Protocol == ELiveLinkOpenTrackIONetworkProtocol::Multicast",
		EditConditionHides
		))
	FString MulticastGroup = TEXT("239.135.1.1");

	/** Multicast Port. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (
		EditCondition = "Protocol == ELiveLinkOpenTrackIONetworkProtocol::Multicast",
		EditConditionHides,
		ClampMin = "1", ClampMax = "65535",
		UIMin = "1", UIMax = "65535"
		))
	uint16 MulticastPort = 55555;

	/** Local interface (used in both Unicast and Multicast protocols). */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FString LocalInterface = TEXT("0.0.0.0");

	/** Unicast Port. 0 means self-assign. Use a port number available in your system. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (
		EditCondition = "Protocol == ELiveLinkOpenTrackIONetworkProtocol::Unicast",
		EditConditionHides
		))
	uint16 UnicastPort = 0;

};
