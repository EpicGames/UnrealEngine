// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenTrackIOConversions.h"
#include "CameraCalibrationSubsystem.h"
#include "Models/BrownConradyDULensModel.h"
#include "Models/BrownConradyUDLensModel.h"
#include "Engine/Engine.h"

namespace LiveLinkOpenTrackIOConversions
{
	namespace // Anonymous namespace for implementation details private to this file
	{
		
		// Convert OpenTrackIO parameters layout and unit to Unreal's.
		void ConvertBrownConradyOTRKToOpenCV(
			const TArray<float>& InRadial,
			const TArray<float>& InTangential,
			float FocalLength_mm,
			TArray<float>& OutParameters
		)
		{
			// Ensure we have the expected number of parameters
			// OpenTrackIO: 6 radial (l1-l6) + 2 tangential (q1-q2)
			// Unreal OpenCV: 6 radial (k1-k6) + 2 tangential (p1-p2)
			
			OutParameters.Empty(8);
			
			// Convert radial coefficients
			// OTRK l1,l2,l3,l4,l5,l6 -> OpenCV k1,k4,k2,k5,k3,k6
			// Note that both the layout and the units are different.
			if (InRadial.Num() >= 6 && FocalLength_mm > KINDA_SMALL_NUMBER)
			{
				const float F2 = FocalLength_mm * FocalLength_mm;
				const float F4 = F2 * F2;
				const float F6 = F4 * F2;
				
				// Map to Unreal's k1-k6 order (numerator k1,k2,k3, denominator k4,k5,k6)
				OutParameters.Add(InRadial[0] * F2);  // k1 = l1 * f^2
				OutParameters.Add(InRadial[2] * F4);  // k2 = l3 * f^4
				OutParameters.Add(InRadial[4] * F6);  // k3 = l5 * f^6
				OutParameters.Add(InRadial[1] * F2);  // k4 = l2 * f^2
				OutParameters.Add(InRadial[3] * F4);  // k5 = l4 * f^4
				OutParameters.Add(InRadial[5] * F6);  // k6 = l6 * f^6
			}
			else
			{
				// Default to zero if parameters are missing
				for (int32 Idx = 0; Idx < 6; ++Idx)
				{
					OutParameters.Add(0.0f);
				}
			}
			
			// Convert tangential coefficients
			if (InTangential.Num() >= 2 && FocalLength_mm > KINDA_SMALL_NUMBER)
			{
				OutParameters.Add(InTangential[0] * FocalLength_mm);  // p1 = q1 * f
				OutParameters.Add(InTangential[1] * FocalLength_mm);  // p2 = q2 * f
			}
			else
			{
				// Default to zero if parameters are missing
				OutParameters.Add(0.0f);
				OutParameters.Add(0.0f);
			}
		}
		
		
		// Check if the model requires parameter conversion
		bool RequiresParameterConversion(const FName& OTRKModelName)
		{
			return (OTRKModelName == UE::OpenTrackIO::BrownConradyDU || OTRKModelName == UE::OpenTrackIO::BrownConradyUD);
		}
	}

	void ToUnrealLens(
		FLiveLinkLensFrameData& OutUnrealLensData, 
		const FLiveLinkOpenTrackIOLens* InLensData,
		const FLiveLinkOpenTrackIOStaticCamera* InCamera
	)
	{
		// FIZ
		if (InLensData)
		{
			if (InLensData->FocusDistance.IsSet())
			{
				OutUnrealLensData.FocusDistance = InLensData->FocusDistance.GetValue() * MetersToCentimeters;
			}

			if (InLensData->FStop.IsSet())
			{
				OutUnrealLensData.Aperture = InLensData->FStop.GetValue();
			}

			if (InLensData->PinholeFocalLength.IsSet())
			{
				OutUnrealLensData.FocalLength = InLensData->PinholeFocalLength.GetValue();  // Both are in mm
			}
		}

		// Filmback
		if (InCamera)
		{
			if (InCamera->ActiveSensorPhysicalDimensions.Height.IsSet())
			{
				OutUnrealLensData.FilmBackHeight = InCamera->ActiveSensorPhysicalDimensions.Height.GetValue(); // Both are in mm
			}

			if (InCamera->ActiveSensorPhysicalDimensions.Width.IsSet())
			{
				OutUnrealLensData.FilmBackWidth = InCamera->ActiveSensorPhysicalDimensions.Width.GetValue();   // Both are in mm
			}
		}

		// Lens Distortion
		//
		// Only valid if the normalizing parameters are present (OpenTrackIO filmback and focal length, both in mm).
		if (InCamera && InLensData 
			&& InCamera->ActiveSensorPhysicalDimensions.Height.IsSet()
			&& InCamera->ActiveSensorPhysicalDimensions.Width.IsSet()
			&& InLensData->PinholeFocalLength.IsSet()
			)
		{
			const float Width_mm = InCamera->ActiveSensorPhysicalDimensions.Width.GetValue();
			const float Height_mm = InCamera->ActiveSensorPhysicalDimensions.Height.GetValue();
			const float F_mm = InLensData->PinholeFocalLength.GetValue();

			if ((Width_mm > KINDA_SMALL_NUMBER) && (Height_mm > KINDA_SMALL_NUMBER))  // Avoid division by zero.
			{
				// Cx Cy
				{
					// OpenTrackIO ProjectionOffset is in mm, Unreal PrincipalPoint is normalized 0..1, centered at 0.5.
					// We normalize ProjectionOffset in mm by the filmback dimensions that are also in mm.

					OutUnrealLensData.PrincipalPoint.X = 0.5 + InLensData->ProjectionOffset.X / Width_mm;
					OutUnrealLensData.PrincipalPoint.Y = 0.5 + InLensData->ProjectionOffset.Y / Height_mm;
				}

				// FxFy
				{
					// OpenTrackIO specifies a singular focal length F. We normalize by the filmback dimensions to get
					// focal length in UV units. That is, if they multiply by 3d coordinates normalized by depth,
					// we get normalized screen coordiantes in the range [0,1].
					OutUnrealLensData.FxFy.X = F_mm / Width_mm;
					OutUnrealLensData.FxFy.Y = F_mm / Height_mm;
				}

				// Distortion parameters
				// Note: The model name was already captured in the Live Link Static Data.
				if (InLensData->Distortion.Num() > 0)
				{
					// For now we will pick the first model in the array.
					const FLiveLinkOpenTrackIOLens_DistortionCoeff& Distortion = InLensData->Distortion[0];
					// Get the OpenTrackIO model name (use default if not specified)
					FName OTRKModelName = Distortion.Model.IsNone() ? UE::OpenTrackIO::BrownConradyDU : Distortion.Model;
					
					// Check if this is a Brown-Conrady model that needs conversion
					if (RequiresParameterConversion(OTRKModelName))
					{
						ConvertBrownConradyOTRKToOpenCV(
							Distortion.Radial,
							Distortion.Tangential,
							F_mm,
							OutUnrealLensData.DistortionParameters
						);
					}
					else
					{
						// For other models, copy all parameters as-is
						OutUnrealLensData.DistortionParameters.Empty();
						
						OutUnrealLensData.DistortionParameters.Append(Distortion.Radial.GetData(), Distortion.Radial.Num());
						OutUnrealLensData.DistortionParameters.Append(Distortion.Tangential.GetData(), Distortion.Tangential.Num());
						OutUnrealLensData.DistortionParameters.Append(Distortion.Custom.GetData(), Distortion.Custom.Num());
					}
				}
			}
		}
	}

}