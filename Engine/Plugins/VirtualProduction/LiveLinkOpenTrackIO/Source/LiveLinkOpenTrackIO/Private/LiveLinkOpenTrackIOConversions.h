// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LiveLinkLensTypes.h"
#include "LiveLinkOpenTrackIOTypes.h"


/** Conversion helper functions from OpenTrackIO coordinates and units to Unreal */
namespace LiveLinkOpenTrackIOConversions
{
	/** OpenTrackIO coordinate system
	 *
	 * OpenTrackIO is:
	 * o Z-up
	 * o Y-forward
	 * o Right-handed
	 *
	 * Unreal is:
	 * o Z-up
	 * o X-forward
	 * o Left-handed
	 *
	 * To convert translation and scale, swap X and Y, which both updates the forward vector and toggles the handedness.
	 * Translation also needs a factor of 100 because OpenTrackIO units are meters, while Unreal uses centimeters.
	 * To convert rotations, invert the sign of Pan/Yaw. The others can stay the same.
	 * Rotation units are degrees in both OpenTrackIO and Unreal.
	 */

	/** Factor to convert from OpenTrackIO's meters to Unreal's centimeters */
	inline constexpr float MetersToCentimeters = 100.f;


	/** Populate Live Link Lens Frame Data from relevant OpenTrackIO structures */
	void ToUnrealLens(
		FLiveLinkLensFrameData& OutUnrealLensData, 
		const FLiveLinkOpenTrackIOLens* InLensData,
		const FLiveLinkOpenTrackIOStaticCamera* InCamera
	);

	/** From OpenTrackIO to Unreal Translation: Swap X <-> Y and convert meters to cm */
	inline FVector ToUnrealTranslation(const FLiveLinkOpenTrackIO_XYZ& InXYZ)
	{
		return FVector(InXYZ.Y, InXYZ.X, InXYZ.Z) * MetersToCentimeters;
	}

	/** From OpenTrackIO to Unreal Scale: Swap X <-> Y (unitless) */
	inline FVector ToUnrealScale(const FLiveLinkOpenTrackIO_XYZ& InXYZ)
	{
		return FVector(InXYZ.Y, InXYZ.X, InXYZ.Z);
	}

	/** From OpenTrackIO to Unreal Translation: Invert sign of Yaw */
	inline FRotator ToUnrealRotation(const FLiveLinkOpenTrackIO_Rotator& InRotator)
	{
		return FRotator(InRotator.Tilt, -InRotator.Pan, InRotator.Roll);
	}

	/** From OpenTrackIO to Unreal FTransform */
	inline FTransform ToUnrealTransform(const FLiveLinkOpenTrackIOTransform& InTransform)
	{
		return FTransform(
			ToUnrealRotation(InTransform.Rotation),
			ToUnrealTranslation(InTransform.Translation),
			ToUnrealScale(InTransform.Scale)
		);
	}
}

