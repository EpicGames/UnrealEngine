// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"

#include "LiveLinkOpenTrackIODatagram.generated.h"

/**
 * This struct stores the header portion of a UDP or serial OpenTrackIO payload. It only contains the header portion and the payload part will be
 * handled separately via payload parsers.
 *
 * See specification at https://www.opentrackio.org/
 */
USTRUCT()
struct FLiveLinkOpenTrackIODatagramHeader
{
	GENERATED_BODY()

	/**
	 * Bit offset 0-31
	 * 4 bytes: Static value to indicate OpenTrackIO packet, set to ASCII "OTrk" (0x4F54726B)
	 */ 
	UPROPERTY()
	uint32 Identifier = 0;

	/**
	 * Bit offset 32-39
	 * 1 byte: This field is reserved for future use and should be ignored by both Producers and Consumers.
	 */
	UPROPERTY()
	uint8 Reserved = 0;

	/**
	 * Bit offset 40-47
	 * 1 byte: Indicates the payload format (e.g., JSON = 0x01, CBOR = 0x02, OTP = 0x02). 0x80 and above are reserved for vendor specific protocols
	 */
	UPROPERTY()
	uint8 Encoding = 0;

	/**
	 * Bit offset 48-63
	 * 2 bytes: A 16-bit unsigned integer indicating the OpenTrackIO packet's unique sequence number (0x01 to UINT16)
	 */
	UPROPERTY()
	uint16 SequenceNumber = 0;

	/**
	 * Bit offset 64-95
	 * 4 bytes: A 32-bit field indicating the byte offset of this payload segment when the overall payload length necessitates segmentation. Must be set to 0x00 for single-segment payloads.
	 */
	UPROPERTY()
	uint32 SegmentOffset = 0;

	/**
	 * Bit offset 96-111
	 * The first bit shall be set to 1 if this is the only segment or the last segment in a segmented payload, or 0 if more segments are expected. The rest of the bits represent
	 * the total lenght of the payload.
	 *
	 * This differs slightly from the OpenTrackIO spec in that they have broken out LastSegment and Payload as separate fields but to use as a USTRUCT we must lump them together.
	 */
	UPROPERTY()
	uint16 LastSegmentFlagAndPayloadLength = 0;
	
	/**
	 * Bit offset 112-127
	 * 2 bytes: A 16-bit checksum computed using the Fletcher-16 algorithm, covering the header (excluding checksum bytes) and payload.
	 */
	UPROPERTY()
	uint16 Checksum = 0;

	/**
	 * Returns true if the payload is complete and can be parsed.  If the payload is segmented then this will return false. 
	 */
	bool IsLastSegment() const
	{
		return LastSegmentFlagAndPayloadLength & 0x8000;
	}

	/** Helper function to return the size of the payload by and-ing the last 15 bits. */
	uint16 GetPayloadSize() const
	{
		static_assert(sizeof(FLiveLinkOpenTrackIODatagramHeader) == 16, "OpenTrackIO header is expected to be 16 bytes.");
		
		return LastSegmentFlagAndPayloadLength & 0x7FFF;
	}
	
};

