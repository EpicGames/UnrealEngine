// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkLensTypes.h"

#include "LiveLinkOpenTrackIOTypes.h"

#include "LiveLinkOpenTrackIOLiveLinkTypes.generated.h"

/**
 * Struct for static OpenTrackIO data
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStaticData : public FLiveLinkLensStaticData
{
	GENERATED_BODY()
};

/**
 * Struct for dynamic (per-frame) OpenTrackIO data
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOFrameData : public FLiveLinkLensFrameData
{
	GENERATED_BODY()

	/* Access to the received OpenTrackIO data */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOData OpenTrackData;
};

/**
 * Facility structure to handle OpenTrackIO data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()
		
	/** Static data that should not change every frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkOpenTrackIOStaticData StaticData;

	/** Dynamic data that can change every frame  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkOpenTrackIOFrameData FrameData;
};
