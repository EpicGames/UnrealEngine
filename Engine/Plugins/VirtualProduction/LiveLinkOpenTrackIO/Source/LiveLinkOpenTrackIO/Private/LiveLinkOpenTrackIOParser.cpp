// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "LiveLinkOpenTrackIOParser.h"

#include "HAL/UnrealMemory.h"
#include "LiveLinkOpenTrackIO.h"

#include "LiveLinkOpenTrackIODatagram.h"
#include "LiveLinkOpenTrackIOTypes.h"
#include "OpenTrackIOCborStructDeserializerBackend.h"

#include "JsonObjectConverter.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/MemoryReader.h"

#include "Misc/ByteSwap.h"
#include "Misc/FileHelper.h"

#include "StructSerializer.h"
#include "StructDeserializer.h"

#include "HAL/IConsoleManager.h"

// Payload size limiter to make sure senders don't endlessly send us data without a last segment. 
static float PayloadSizeLimitInMb = 64;
static FAutoConsoleVariableRef  CVarPayloadSizeLimitInMb(TEXT("OpenTrackIO.PayloadSizeLimit"), PayloadSizeLimitInMb, TEXT("Limits sizes of payloads that we will accept. The value is in megabytes."));

// Payload size limiter to make sure senders don't endlessly send us data without a last segment. 
static int32 ChecksumModulo = 256;
static FAutoConsoleVariableRef CVarChecksumModulo(TEXT("OpenTrackIO.ChecksumModulo"), ChecksumModulo, TEXT("The modulo value to use in the fletcher-16 checksum value."));

void FLiveLinkOpenTrackIOPayload::AddBytes(uint32 InOffset, TArrayView<const uint8> InBytes)
{
	const float PayloadSizeLimitInFloatBytes = PayloadSizeLimitInMb * 1024 * 1024;
	const uint32 PayloadLimitBytes = static_cast<uint32>(PayloadSizeLimitInFloatBytes);

	if ((PayloadBytes.Num() + InOffset + InBytes.Num()) > PayloadLimitBytes)
	{
		bValidPayload = false;
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Payload size limit exceeded %f MB. Use OpenTrackIO.PayloadSizeLimit to adjust this limit.", PayloadSizeLimitInMb);
		return;
	}
	Sections.Add(FSegmentSection{ InOffset, InBytes.Num() });

	// Preallocate and set the number to the new offset and size.
	PayloadBytes.SetNum(InOffset + InBytes.Num());
	
	FMemory::Memcpy(PayloadBytes.GetData()+InOffset, InBytes.GetData(), InBytes.Num());
}

bool FLiveLinkOpenTrackIOPayload::IsComplete() const
{
	if (!bValidPayload)
	{
		return false;
	}
	
	int32 TotalBytes = 0;

	for (const FSegmentSection& Section : Sections)
	{
		TotalBytes += Section.Size;
	}
	
	if (TotalBytes != PayloadBytes.Num())
	{
		// No possible way it is complete because TotalBytes should match our byte array size.
		return false;
	}

	// If the size is equal then we need to make sure there are no overlaps between the received segments. If we did
	// find an overlap then we can't consider it complete.
	Algo::Sort(
		Sections,
		[](const FSegmentSection& A, const FSegmentSection& B)
		{
			return A.Offset < B.Offset;
		}
	);

	bool bOverlaps = false;
	// Check for overlaps
	for (int32 Index = 1; !bOverlaps && Index < Sections.Num(); Index++)
	{
		if ((Sections[Index - 1].Offset + Sections[Index - 1].Size) < Sections[Index].Offset)
		{
			bOverlaps = true;
			break;
		}
	}
	return !bOverlaps;
}

namespace UE::OpenTrackIO::Private
{

/**
 * Tries to parse our custom Optional USTRUCT from JSON.
 * 
 * @param JsonValue   The JSON value to import.
 * @param Property    The TOptional property to write into.
 * @param Value       Pointer to memory holding the optional property value.
 * 
 * @return true on success.
 */
static bool TryReadStructOptional(
	const TSharedPtr<FJsonValue>& JsonValue,
	FProperty* Property,
	void* ContainerPtr)
{
	// See if this is a struct property

	FStructProperty* StructProperty = CastField<FStructProperty>(Property);

	if (!StructProperty)
	{
		return false;
	}

	// Make sure it is one of our toptional replacement structs.

	UScriptStruct* ScriptStruct = StructProperty->Struct;

	if (!IsOpenTrackIOOptionalType(ScriptStruct))
	{
		return false;
	}

	// Find the 'Value' member

	FProperty* ValueProperty = ScriptStruct->FindPropertyByName(OptionalTypeValueName);

	if (!ValueProperty)
	{
		return false;
	}

	// Only handle numeric inner properties

	FNumericProperty* NumericProperty = CastField<FNumericProperty>(ValueProperty);

	if (!NumericProperty)
	{
		return false;
	}

	// Get pointer to the numeric field inside the struct

	if (!ContainerPtr)
	{
		return false;
	}

	void* NumericPtr = NumericProperty->ContainerPtrToValuePtr<void>(ContainerPtr);

	if (!NumericPtr)
	{
		return false;
	}

	// Set numeric value

	if (NumericProperty->IsFloatingPoint())
	{
		NumericProperty->SetFloatingPointPropertyValue(NumericPtr, JsonValue->AsNumber());
	}
	else if (NumericProperty->IsInteger())
	{
		if (JsonValue->Type == EJson::String)
		{
			// parse string -> int64 ourselves so we don't lose any precision going through AsNumber (aka double)
			NumericProperty->SetIntPropertyValue(NumericPtr, FCString::Atoi64(*JsonValue->AsString()));
		}
		else
		{
			NumericProperty->SetIntPropertyValue(NumericPtr, (int64)JsonValue->AsNumber());
		}
	}
	else
	{
		return false;
	}

	// Mark bIsSet = true

	FBoolProperty* BoolProp = CastField<FBoolProperty>(ScriptStruct->FindPropertyByName(OptionalTypeIsSetName));

	if (BoolProp)
	{
		BoolProp->SetPropertyValue_InContainer(ContainerPtr, true);
	}

	return true;
}

/**
 * Tries to parse a TOptional numeric property from JSON.
 * 
 * @param JsonValue   The JSON value to import.
 * @param Property    The TOptional property to write into.
 * @param Value       Pointer to memory holding the optional property value.
 * 
 * @return true if successful.
 */
static bool TryReadTOptional(
	const TSharedPtr<FJsonValue>& JsonValue,
	FProperty* Property,
	void* Value)
{
	FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property);

	if (!OptionalProperty)
	{
		return false;
	}

	// Handle numeric optional

	FNumericProperty* NumericProperty = CastField<FNumericProperty>(OptionalProperty->GetValueProperty());

	if (!NumericProperty)
	{
		return false;
	}

	// Handle floats and double property types.

	if (NumericProperty->IsFloatingPoint())
	{
		void* OptionalValue = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(Value);

		if (!OptionalValue)
		{
			return false;
		}

		NumericProperty->SetFloatingPointPropertyValue(OptionalValue, JsonValue->AsNumber());

		return true;
	}

	// Handle integer property types.

	if (NumericProperty->IsInteger())
	{
		void* OptionalValue = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(Value);

		if (!OptionalValue)
		{
			return false;
		}

		if (JsonValue->Type == EJson::String)
		{
			// parse string -> int64 ourselves so we don't lose any precision going through AsNumber (aka double)
			NumericProperty->SetIntPropertyValue(OptionalValue, FCString::Atoi64(*JsonValue->AsString()));
		}
		else
		{
			NumericProperty->SetIntPropertyValue(OptionalValue, (int64)JsonValue->AsNumber());
		}

		return true;
	}

	// We did not write into this type, so we return false for the default parser to handle it.
	return false;
}

/**
* Callback for importing JSON values into TOptional (currently numeric) properties,
* or our custom OptionalStruct types that can be natively exposed to Blueprints.
*
* @param JsonValue   The JSON value to import.
* @param Property    The TOptional property to write into.
* @param Value       Pointer to memory holding the optional property value.
* 
* @return            True if this callback handled the import (optional set successfully).
*/
static bool JsonOptionalImporterCallback(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value)
{
	// Try struct-based optionals
	if (TryReadStructOptional(JsonValue, Property, Value))
	{
		return true;
	}

	// Try TOptional
	return TryReadTOptional(JsonValue, Property, Value);
}

TOptional<FLiveLinkOpenTrackIOData> ParseJsonBlob(const FString& JsonBlob)
{
	TSharedPtr<FJsonObject> JsonObject;
	FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(*JsonBlob), JsonObject);

	TOptional<FLiveLinkOpenTrackIOData> OutData;
	if (!JsonObject)
	{
		return OutData;
	}

	TSharedPtr<FJsonValue> StaticData = JsonObject->TryGetField(TEXT("static"));
	TSharedPtr<FJsonValue> LensData = JsonObject->TryGetField(TEXT("lens"));
	TSharedPtr<FJsonValue> TrackerData = JsonObject->TryGetField(TEXT("timing"));
	TSharedPtr<FJsonValue> ProtocolData = JsonObject->TryGetField(TEXT("protocol"));

	/** We need to have at least one of these objects in the payload. */
	if (!StaticData && !LensData && !TrackerData && !ProtocolData)
	{
		return OutData;
	}
	
	FLiveLinkOpenTrackIOData Data;

	// We use the json importer callback to add support of optional properties.
	FJsonObjectConverter::CustomImportCallback OptionalNumericImporter;
	OptionalNumericImporter.BindStatic(JsonOptionalImporterCallback);

	if (FJsonObjectConverter::JsonObjectStringToUStruct(JsonBlob, &Data, 0, 0, false, nullptr, &OptionalNumericImporter))
	{
		OutData.Emplace(MoveTemp(Data));
	}
	return OutData;
}

TOptional<FLiveLinkOpenTrackIOData> ParseCborBlob(TArrayView<const uint8> InBytes)
{
	TOptional<FLiveLinkOpenTrackIOData> OutData;
	if (InBytes.IsEmpty())
	{
		return OutData;
	}
	
	FMemoryReaderView Reader(InBytes);
	FOpenTrackIOCborStructDeserializerBackend Deserializer(Reader);

	UScriptStruct* InType = TBaseStructure<FLiveLinkOpenTrackIOData>::Get();

	FLiveLinkOpenTrackIOData Data;
	const bool bReadCbor = FStructDeserializer::Deserialize(&Data, *InType, Deserializer) && !Reader.GetError();

	if (bReadCbor)
	{
		OutData.Emplace(MoveTemp(Data));
	}

	return OutData;
}

/**
 * OpenTrackIO specifies using Fletcher-16 algorithm for checksum. 
 * Optimized version from: https://en.wikipedia.org/wiki/Fletcher%27s_checksum
 */
static uint16_t CalculateChecksum(TArrayView<const uint8> HeaderBytes, TArrayView<const uint8> PayloadBytes)
{
	uint32_t C0 = 0;
	uint32_t C1 = 0;

	auto CalcFletcher16Chunk = [&](const uint8* Data, uint32 Len)
		{
			while (Len > 0)
			{
				uint32 BlockLen = Len > 5802 ? 5802 : Len;

				Len -= BlockLen;

				do {
					C0 += *Data++;
					C1 += C0;
				} while (--BlockLen);

				C0 %= ChecksumModulo;
				C1 %= ChecksumModulo;
			}
		};

	CalcFletcher16Chunk(HeaderBytes.GetData(), HeaderBytes.Num());
	CalcFletcher16Chunk(PayloadBytes.GetData(), PayloadBytes.Num());

	return static_cast<uint16>((C1 << 8) | C0);
}

/**
 * These are the defined values in the OpenTrackIO specification.
 */
const uint32 OpenTrackIOHeaderId = 0x4F54726B;
const uint8 JSON_Encoding = 0x01;
const uint8 CBOR_Encoding = 0x02;

FORCEINLINE uint16 ByteSwap(uint16 InVal)
{
#if PLATFORM_LITTLE_ENDIAN
	return BYTESWAP_ORDER16(InVal);
#else
	return InVal;
#endif
}

FORCEINLINE uint32 ByteSwap(uint32 InVal)
{
#if PLATFORM_LITTLE_ENDIAN
	return BYTESWAP_ORDER32(InVal);
#else
	return InVal;
#endif
}
	
inline FArchive& operator<<(FArchive& Ar, FLiveLinkOpenTrackIODatagramHeader& Header)
{
	
	uint32 Identifier;
	Ar << Identifier;
	Header.Identifier = ByteSwap(Identifier);

	// 1 byte values.  No byte swapping necessary.
	Ar << Header.Reserved;
	Ar << Header.Encoding;

	uint16 SequenceNumber;
	Ar << SequenceNumber;
	Header.SequenceNumber = ByteSwap(SequenceNumber);

	uint32 SegmentOffset;
	Ar << SegmentOffset;
	Header.SegmentOffset = ByteSwap(SegmentOffset);

	uint16 SegmentAndPayloadLength;
	Ar << SegmentAndPayloadLength;
	Header.LastSegmentFlagAndPayloadLength = ByteSwap(SegmentAndPayloadLength);

	uint16 Checksum;
	Ar << Checksum;
	Header.Checksum = ByteSwap(Checksum);
	
	return Ar;
}

static bool IsHeaderValid(TArrayView<const uint8> Bytes)
{
	const uint32 HeaderSize = sizeof(FLiveLinkOpenTrackIODatagramHeader);
	/** No way this is a valid datagram if we can't fit it in our header. */
	if (Bytes.Num() < HeaderSize)
	{
		return false;
	}

	/** Check the header for validity first before trying to read the rest of the packet. */
	{
		FMemoryReaderView Reader(Bytes);
		
		uint32 Identifier;
		Reader << Identifier;
		Identifier = ByteSwap(Identifier);
		if (Identifier != OpenTrackIOHeaderId)
		{
			UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Invalid OpenTrackIO header. It was %08X instead of %08X",
				Identifier, OpenTrackIOHeaderId
			);

			return false;
		}
	}
	return true;
}

static TOptional<FLiveLinkOpenTrackIOData> ConvertPayloadToData(const FLiveLinkOpenTrackIODatagramHeader& Header, TArrayView<const uint8> Payload)
{
	/** Now attempt to parse the payload using either JSON or CBOR. */
	TOptional<FLiveLinkOpenTrackIOData> Data;
	if (Header.Encoding == JSON_Encoding)
	{
		FString JsonBlob;
		FFileHelper::BufferToString(JsonBlob, Payload.GetData(), Payload.Num());
		Data = ParseJsonBlob(JsonBlob);
	}
	else if (Header.Encoding == CBOR_Encoding)
	{
		Data = ParseCborBlob(Payload);
	}
	else
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Unsupported encoding.");
	}

	return Data;
}

/**
 * Helper function that takes header and payload data and converts it into a FLiveLinkOpenTrackIOData data struct.
 *
 * If this fails then the optional will be unset. 
 */
static TOptional<FLiveLinkOpenTrackIOData> TryEmitOpenTrackData(FLiveLinkOpenTrackIODatagramHeader Header, TArrayView<const uint8> Payload)
{
	TOptional<FLiveLinkOpenTrackIOData> Data = ConvertPayloadToData(Header, Payload);
	if (Data && Data->Protocol.IsSupported())
	{
		// Manually compensate for OpenTrackIO not explicitly stating the number of frames per frame.
		{
			// Calculate from sampleRate / timecodeRate, because it tells us how many samples there are per timecode frame.
			const int32 RatioNum = Data->Timing.SampleRate.Num * Data->Timing.Timecode.FrameRate.Denom;
			const int32 RatioDen = FMath::Max(1, Data->Timing.SampleRate.Denom * Data->Timing.Timecode.FrameRate.Num);
			
			// Enforce a minium of 1 SubframesPerFrame, which is equivalent to no frame subdivision.
			Data->Timing.Timecode.SubframesPerFrame = FMath::Max(1, FMath::DivideAndRoundNearest(RatioNum, RatioDen));
		}
	}
	else if (Data)
	{
		// We successfully parsed a data packet but we were unable to support the protocol.
		//
		Data.Reset();
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Unsupported OpenTrackIO protocol.");
	}

	return Data;
}

bool GetHeaderAndPayloadFromBytes(TArrayView<const uint8> Bytes, FOpenTrackIOHeaderWithPayload& OutPayloadContainer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OpenTrackIO::GetHeaderAndPayloadFromBytes);

	const uint32 HeaderSize = sizeof(FLiveLinkOpenTrackIODatagramHeader);
	if (!IsHeaderValid(Bytes))
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Invalid OpenTrackIO Header Received.");
		return false;		
	}

	FLiveLinkOpenTrackIODatagramHeader& Header = OutPayloadContainer.GetMutableHeader();
	uint16 PrevSequenceNumber = Header.SequenceNumber;
	
	/** Now read in the header to prepare parsing the payload. */
	FMemoryReaderView Reader(Bytes);
	Reader << Header;

	/** Confirm that the payload size is within the number of bytes given to us. */
	uint16 PayloadSize = Header.GetPayloadSize();
	if (PayloadSize > Bytes.Num())
	{
		/** The header does not match the inbound data. Abort! */
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Payload size does not match provided OpenTrackIO packet header.");
		return false;
	}

	TArrayView<const uint8> HeaderMinusChecksum = Bytes.Slice(0, HeaderSize-2);
	TArrayView<const uint8> Payload = Bytes.Slice(HeaderSize, PayloadSize);

	/** Use a fletcher-16 algorithm to calculate the checksum of the data. */
	uint16_t Checksum = CalculateChecksum(HeaderMinusChecksum, Payload);
	if (Checksum != Header.Checksum)
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Failed to verify packet checksum.");
		return false;
	}

	// According to the OpenTrackIO team, he sequence number will only increase when a full payload has been received. So the sequence number
	// should match the previous sequence number.
	FLiveLinkOpenTrackIOPayload& PayloadStorage = OutPayloadContainer.GetMutablePayload();
	if (PayloadStorage.HasAnyPayloadData() && Header.SequenceNumber != PrevSequenceNumber)
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Invalid sequence number for segmented payload. Expected %d but received %d.", PrevSequenceNumber, Header.SequenceNumber);
		return false;
	}
	
	PayloadStorage.AddBytes(Header.SegmentOffset, Payload);

	return true;
}

TOptional<FLiveLinkOpenTrackIOData> ParsePayload(const FOpenTrackIOHeaderWithPayload& HeaderAndPayload)
{
	const FLiveLinkOpenTrackIODatagramHeader& Header = HeaderAndPayload.GetHeader(); 
	check(HeaderAndPayload.GetHeader().IsLastSegment());
	
	const FLiveLinkOpenTrackIOPayload& Payload = HeaderAndPayload.GetPayload();
	if (!Payload.IsComplete())
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Discarding a partial payload because of missing segments.");
		return {};
	}

	return TryEmitOpenTrackData(Header, Payload.GetBytes());
}

}


