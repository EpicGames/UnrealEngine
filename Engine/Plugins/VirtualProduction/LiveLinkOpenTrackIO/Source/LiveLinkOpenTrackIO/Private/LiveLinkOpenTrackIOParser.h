// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkOpenTrackIOTypes.h"
#include "LiveLinkOpenTrackIODatagram.h"

#include "Containers/ArrayView.h"
#include "Containers/Array.h"

/**
 * Payload storage for inbound data stream.
 *
 * If the datagram header indicates a payload will be transmitted over multiple units, we store payload
 * data by calling AddBytes per new packet arrival until all packets are received (last segment flag on the header).
 *
 * There is no reliable transmission protocol for OpenTrackIO. If we detect that the entire payload was not
 * delivered by the time we receive the last segment flag then we throw out the entire payload. 
 */
struct FLiveLinkOpenTrackIOPayload
{
	FLiveLinkOpenTrackIOPayload() = default;

	/** Add a partial payload to this class. */
	void AddBytes(uint32 InOffset, TArrayView<const uint8> InBytes);

	/**
	 * Inspect all data received and confirm that it is complete.
	 * 
	 * To qualify as completed, we must have no gaps in our received data and the payload must not have exceeded
	 * our payload limit set in OpenTrackIO.PayloadLimitSize
	 */
	bool IsComplete() const;

	/**
	 * Returns true if any payload data exists on this struct. This is useful for detecting continuity of
	 * data between header values. 
	 */
	bool HasAnyPayloadData() const
	{
		return Sections.Num() > 0;
	}
	
	/**
	 * Return a view into the data such that it can be passed into the CBOR or JSON parsers.
	 *
	 * You must ensure that you check that the payload is complete. 
	 */
	TArrayView<const uint8> GetBytes() const
	{
		return PayloadBytes;
	}

private:

	/** Tracks if the payload is valid across all segments.  If we exceed our maximum payload size or we have received duplicate values */
	bool bValidPayload = true;

	/** Each segment received with Offset / Size pair. */
	struct FSegmentSection
	{
		/** InOffset value */
		uint32 Offset;

		/** Size*/
		int32 Size;
	};

	/**
	 * Array to track all of the segment sections added to this payload.
	 *
	 * It is tagged mutable to support sorting within IsComplete() method. 
	 */
	mutable TArray<FSegmentSection> Sections;

	/** Storage buffer for payload data from a packet stream. */
	TArray<uint8> PayloadBytes;
};


struct FOpenTrackIOHeaderWithPayload
{
	/**
	 * Return a const reference to the header.
	 */
	const FLiveLinkOpenTrackIODatagramHeader& GetHeader() const
	{
		return Header;
	}

	/**
	 * Return a const reference to the payload data. 
	 */
	const FLiveLinkOpenTrackIOPayload& GetPayload() const
	{
		return Payload;
	}

	/**
	 * Read / write access to the header.
	 */
	FLiveLinkOpenTrackIODatagramHeader& GetMutableHeader()
	{
		return Header;
	}

	/**
	 * Read / write access to the payload datga. 
	 */
	FLiveLinkOpenTrackIOPayload& GetMutablePayload()
	{
		return Payload;
	}


private:
	FLiveLinkOpenTrackIODatagramHeader Header;
	FLiveLinkOpenTrackIOPayload Payload;
};

namespace UE::OpenTrackIO::Private
{
	/**
	 * Try to parse a JSON string and return an optional FLiveLinkOpenTrackIOData type.  If parsing fails
	 * an unset optional will be returned.  If parsing succeeds then data will be returned.
	 *
	 * It is assumed that the JSON blob conforms to the minimal JSON. If it does not conform to the
	 * minimal JSON then an unset optional will be returned.
	 */
	TOptional<FLiveLinkOpenTrackIOData> ParseJsonBlob(const FString& JsonBlob);

	/**
	 * Try to parse a CBOR blob. Return an optional FLiveLinkOpenTrackIOData type if CBOR parsing succeeds.
	 */
	TOptional<FLiveLinkOpenTrackIOData> ParseCborBlob(TArrayView<const uint8> ByteArray);

	/**
	 * From a byte data, split the byte stream into a header and payload part validating the header and running a Fletcher-16 checksum on the
	 * header and payload.
	 *
	 * The result, if successful, is placed into OutPayloadContainer.  The function will return false if the checksum, header, or payload data
	 * is invalid. 
	 */
	bool GetHeaderAndPayloadFromBytes(TArrayView<const uint8> Bytes, FOpenTrackIOHeaderWithPayload& OutPayloadContainer);

	/**
 	 * Try to parse the payload from the given header and payload data. This will produce a FLiveLinkOpenTrackIOData struct that contains the full
 	 * values read from the payload data. If the parsers fail to read the payload then an unset optional will be returned.
	 */
	TOptional<FLiveLinkOpenTrackIOData> ParsePayload(const FOpenTrackIOHeaderWithPayload& HeaderAndPayload);
}
