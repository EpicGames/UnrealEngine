// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenTrackIORole.h"

#include "LiveLinkOpenTrackIORole.h"
#include "LiveLinkOpenTrackIOLiveLinkTypes.h"

#define LOCTEXT_NAMESPACE "LiveLinkOpenTrackIORole"

UScriptStruct* ULiveLinkOpenTrackIORole::GetStaticDataStruct() const
{
	return FLiveLinkOpenTrackIOStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkOpenTrackIORole::GetFrameDataStruct() const
{
	return FLiveLinkOpenTrackIOFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkOpenTrackIORole::GetBlueprintDataStruct() const
{
	return FLiveLinkOpenTrackIOBlueprintData::StaticStruct();
}

bool ULiveLinkOpenTrackIORole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkOpenTrackIOBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkOpenTrackIOBlueprintData>();
	const FLiveLinkOpenTrackIOStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkOpenTrackIOStaticData>();
	const FLiveLinkOpenTrackIOFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkOpenTrackIOFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkOpenTrackIORole::GetDisplayName() const
{
	return LOCTEXT("OpenTrackIORole", "OpenTrackIO");
}

#undef LOCTEXT_NAMESPACE
