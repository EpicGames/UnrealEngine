// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkLensRole.h"

#include "LiveLinkOpenTrackIORole.generated.h"

/**
 * Role associated with OpenTrackIO data
 */
UCLASS(BlueprintType, meta = (DisplayName = "OpenTrackIO Role"))
class ULiveLinkOpenTrackIORole : public ULiveLinkLensRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	virtual UScriptStruct* GetStaticDataStruct() const override;
	virtual UScriptStruct* GetFrameDataStruct() const override;
	virtual UScriptStruct* GetBlueprintDataStruct() const override;

	virtual bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	virtual FText GetDisplayName() const override;
	//~ End ULiveLinkRole interface
};

