// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenTrackIOSource.h"

#include "Engine/Engine.h"

#include "LiveLinkOpenTrackIOConversions.h"

#include "ILiveLinkClient.h"
#include "LiveLinkTypes.h"

#include "LiveLinkOpenTrackIOLiveLinkTypes.h"
#include "LiveLinkOpenTrackIOTypes.h"

#include "LiveLinkOpenTrackIO.h"
#include "LiveLinkOpenTrackIOParser.h"
#include "LiveLinkOpenTrackIOSourceSettings.h"
#include "LiveLinkOpenTrackIOTranscoder.h"

#include "LiveLinkOpenTrackIORole.h"
#include "Roles/LiveLinkCameraRole.h"

#include "HAL/Platform.h"
#include "Misc/CoreDelegates.h"

#include "Common/UdpSocketBuilder.h"
#include "Common/UdpSocketReceiver.h"

#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "SocketSubsystem.h"
#include "Sockets.h"

#define LOCTEXT_NAMESPACE "LiveLinkOpenTrackIOSource"

namespace UE::LiveLinkOpenTrackIO::Private
{
TOptional<FIPv4Endpoint> ParseEndpoint(const FString& InEndpointString)
{
	FIPv4Endpoint Endpoint;
	bool bParsedAddr = FIPv4Endpoint::Parse(InEndpointString, Endpoint);
	if (!bParsedAddr)
	{
		bParsedAddr = FIPv4Endpoint::FromHostAndPort(InEndpointString, Endpoint);
	}
	if (bParsedAddr)
	{
		// Detect 169.254.x.x addresses.
		if (Endpoint.Address.A == 169 && Endpoint.Address.B == 254)
		{
			UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Detected IPv4 address in the form of 169.254.x.x. This is a link assigned address and may prevent you from reaching external endpoints. ");
		}

		return Endpoint;
	}

	return {};
}

void DoJoinMulticastGroup(const TSharedRef<FInternetAddr>& MulticastAddr, const TSharedPtr<FInternetAddr>& IpAddr, FSocket* MulticastSocket)
{
	if (!IpAddr.IsValid())
	{
		return;
	}
	const bool bJoinedGroup = MulticastSocket->JoinMulticastGroup(*MulticastAddr, *IpAddr);
	if (bJoinedGroup)
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Display, "Added local interface '%ls' to multicast group '%ls'", *IpAddr->ToString(false), *MulticastAddr->ToString(true));
	}
	else
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Failed to join multicast group '%ls' on detected local interface '%ls'", *MulticastAddr->ToString(true), *IpAddr->ToString(false));
	}
}

void JoinedToGroup(const FIPv4Endpoint& UnicastEndpoint, const FIPv4Endpoint& MulticastEndpoint, FSocket* MulticastSocket)
{
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
	TSharedRef<FInternetAddr> MulticastAddr = MulticastEndpoint.ToInternetAddr();
	if (UnicastEndpoint.Address == FIPv4Address::Any)
	{
		TArray<TSharedPtr<FInternetAddr>> LocapIps;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(LocapIps);
		for (const TSharedPtr<FInternetAddr>& LocalIp : LocapIps)
		{
			DoJoinMulticastGroup(MulticastAddr, LocalIp, MulticastSocket);
		}

		// GetLocalAdapterAddresses returns empty list when all network adapters are offline
		if (LocapIps.Num() == 0)
		{
			bool bCanBindAll = false;
			DoJoinMulticastGroup(MulticastAddr, ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll), MulticastSocket);
		}
	}
	else
	{
		TSharedRef<FInternetAddr> UnicastAddr = UnicastEndpoint.ToInternetAddr();
		DoJoinMulticastGroup(MulticastAddr, UnicastAddr, MulticastSocket);
	}
#endif
}

} // namespace UE::LiveLinkOpenTrackIO::Private

FLiveLinkOpenTrackIOSource::FLiveLinkOpenTrackIOSource(FLiveLinkOpenTrackIOConnectionSettings InConnectionSettings)
	: ConnectionSettings(MoveTemp(InConnectionSettings))
{
	SourceType = LOCTEXT("SourceType_OpenTrackIO", "OpenTrack I/O");
	SourceStatus = LOCTEXT("Initialization", "Initializing receivers...");

	// We'll use this callback for cleanup fallback. See FLiveLinkOpenTrackIOSource::OnEnginePreExit for more details.
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FLiveLinkOpenTrackIOSource::OnEnginePreExit);
}

void FLiveLinkOpenTrackIOSource::OnEnginePreExit()
{
	// This source uses latent shutdown (see RequestSourceShutdown()) but because Update() may not be called again to wait for
	// the receiver threads to stop, it might be possible to access UObject subsystems from those threads that are being torn down.
	// e.g. TBaseStructure<FLiveLinkOpenTrackIOData>::Get() will return invalid data and crash the Cbor parser.
	//
	// To avoid this deterministically, on EnginePreExit we request shutdown and make one last call to Update() which should
	// wait for the udp receiver threads to end.

	RequestSourceShutdown();
	Update();
}

FLiveLinkOpenTrackIOSource::~FLiveLinkOpenTrackIOSource()
{
	CloseSockets();

	// Remove our backup cleanup call.
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FLiveLinkOpenTrackIOSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;
}

bool FLiveLinkOpenTrackIOSource::IsSourceStillValid() const
{
	return ConnectionState == ELiveLinkOpenTrackIOState::Receiving;
}

bool FLiveLinkOpenTrackIOSource::RequestSourceShutdown()
{
	if (ConnectionState == ELiveLinkOpenTrackIOState::ShutDown)
	{
		return true;
	}

	// We do a latent shutdown because of a possible deadlock due to a mutex shared between:
	//
	// * PushSubjectFrameData_AnyThread and
	// * RequestSourceShutdown
	//
	// Since PushSubjectFrameData_AnyThread is called in the Udp receiver thread, it means we can't wait for that thread
	// to stop here, because it may be waiting for the lock we're currently in.

	// Stop Udp Receivers so that they stop pushing more packets unnecessarily.
	StopUdpReceivers();

	// This flag will be checked by the state machine, which will then clean up and enter the ShutDown state.
	bShutdownRequested = true;

	return false;
}

void FLiveLinkOpenTrackIOSource::Update()
{
	// Any state can lead to ShutDown directly.
	if (bShutdownRequested && (ConnectionState != ELiveLinkOpenTrackIOState::ShutDown))
	{
		CloseSockets();
		SetConnectionState(ELiveLinkOpenTrackIOState::ShutDown);

		// Note: We purposedly do not clear the bShutdownRequested flag. It is one-time use only,
		// which means that this source will never exit the ShutDown state.
	}
	else if (bResetRequested)
	{
		// Any state, unless shutting down, can lead to ResetRequested.
		SetConnectionState(ELiveLinkOpenTrackIOState::ResetRequested);

		bResetRequested = false;
	}

	switch (ConnectionState)
	{
		case ELiveLinkOpenTrackIOState::NotStarted:
			SourceStatus = LOCTEXT("NotStarted", "Not started");

			// Note: InitializeSettings should get us out of this state directly to ResetRequested
			break;

		case ELiveLinkOpenTrackIOState::ResetRequested:

			SourceStatus = LOCTEXT("Resetting", "Resetting source.");

			// If the user has requested a connection reset. Close the socket and attempt to re-open on the next loop iteration.
			CloseSockets();

			LastDataReadTime = 0;

			// Re-parse the endpoints to reset the connection.
			if (ParseEndpointFromSourceSettings())
			{
				SetConnectionState(ELiveLinkOpenTrackIOState::EndpointsReady);
			}

			break;

		case ELiveLinkOpenTrackIOState::EndpointsReady:
			SourceStatus = LOCTEXT("EndpointsReady", "Starting socket setup.");

			if (OpenSockets())
			{
				SetConnectionState(ELiveLinkOpenTrackIOState::Receiving);
			}
			else
			{
				SetConnectionState(ELiveLinkOpenTrackIOState::ResetRequested);
			}

			break;

		case ELiveLinkOpenTrackIOState::Receiving:
			if (FPlatformTime::Seconds() - LastDataReadTime < 1.0)
			{
				SourceStatus = LOCTEXT("Receiving", "Receiving.");
			}
			else
			{
				SourceStatus = LOCTEXT("WaitingForData", "Waiting for data.");
			}
			break;

		case ELiveLinkOpenTrackIOState::ShutDown:
			SourceStatus = LOCTEXT("ShutDown", "Shut Down");

			// If we're here, then the udp receivers and sockets must have already been closed.
			break;

		default:
			checkNoEntry();
			break;
	};
}

bool FLiveLinkOpenTrackIOSource::ParseEndpointFromSourceSettings()
{
	check(IsValid(SavedSourceSettings));

	using namespace UE::LiveLinkOpenTrackIO::Private;

	// Unicast settings

	// Note: The Unicast port number is not used when using multicast, only the address (which is used as an interface address to bind to).
	UnicastEndpoint.Port = ConnectionSettings.UnicastPort;

	TOptional<FIPv4Endpoint> UniEndpoint = ParseEndpoint(SavedSourceSettings->UnicastEndpoint);

	if (UniEndpoint.IsSet())
	{
		UnicastEndpoint = *UniEndpoint;
	}

	// Multicast Settings

	TOptional<FIPv4Endpoint> MultiEndpoint = ParseEndpoint(SavedSourceSettings->MulticastEndpoint);

	if (MultiEndpoint.IsSet())
	{
		MulticastEndpoint = *MultiEndpoint;
	}

	switch (SavedSourceSettings->Protocol)
	{
		case ELiveLinkOpenTrackIONetworkProtocol::Unicast:

			if (!UniEndpoint.IsSet())
			{
				return false;
			}
			break;

		case ELiveLinkOpenTrackIONetworkProtocol::Multicast:

			if (!UniEndpoint.IsSet() || !MultiEndpoint.IsSet() || !MultiEndpoint->Address.IsMulticastAddress())
			{
				return false;
			}
			break;

		default:
			return false;
	}

	return true;
}

void FLiveLinkOpenTrackIOSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	using namespace UE::LiveLinkOpenTrackIO::Private;

	ILiveLinkSource::InitializeSettings(Settings);

	if (ULiveLinkOpenTrackIOSourceSettings* OpenTrackIOSettings = Cast<ULiveLinkOpenTrackIOSourceSettings>(Settings))
	{
		// Cache this for details to properties with the right context.
		OpenTrackIOSettings->Protocol = ConnectionSettings.Protocol;

		// Multicast
		{
			FIPv4Endpoint Endpoint;

			FIPv4Address::Parse(ConnectionSettings.MulticastGroup, Endpoint.Address);
			Endpoint.Port = ConnectionSettings.MulticastPort;

			OpenTrackIOSettings->MulticastEndpoint = Endpoint.ToString();
		}

		// Unicast
		{
			FIPv4Endpoint Endpoint;

			FIPv4Address::Parse(ConnectionSettings.LocalInterface, Endpoint.Address);
			Endpoint.Port = ConnectionSettings.UnicastPort;

			OpenTrackIOSettings->UnicastEndpoint = Endpoint.ToString();
		}

		SavedSourceSettings = OpenTrackIOSettings;

		bResetRequested = true;
	}
	else
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Invalid source settings.");
	}
}

void FLiveLinkOpenTrackIOSource::HandleInboundData(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& InData, const FIPv4Endpoint& InSender)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OpenTrackIO::HandleInboundData);

	FInboundMessage Message{ InData, InSender };

	if (!Message.Data.IsValid())
	{
		return;
	}

	TArrayView<const uint8> MessageData(Message.Data->GetData(), Message.Data->Num());

	FOpenTrackIOHeaderWithPayload& PayloadContainer = WorkingPayloads.FindOrAdd(Message.Sender);
	if (UE::OpenTrackIO::Private::GetHeaderAndPayloadFromBytes(MessageData, PayloadContainer))
	{
		// Update Source Machine so the user can identify the sending machine.
		if (LastSender_RunnableThread != Message.Sender)
		{
			LastSender_RunnableThread = Message.Sender;
			AsyncTask(ENamedThreads::GameThread, [this, Sender = Message.Sender]() {
				SourceMachineName = Sender.Address.ToText();
			});
		}

		LastDataReadTime = FPlatformTime::Seconds();

		if (PayloadContainer.GetHeader().IsLastSegment())
		{
			{
				// Scoped because want to prevent access to const & Header beyond this use.
				const FLiveLinkOpenTrackIODatagramHeader& Header = PayloadContainer.GetHeader();
				TOptional<FLiveLinkOpenTrackIOData> Data = UE::OpenTrackIO::Private::ParsePayload(PayloadContainer);
				if (Data)
				{
					// This is a sink and we give up FLiveLinkOpenTrackData to optimize moving the OpenTrack data around.
					PushDataToLiveLink_AnyThread(Header, MoveTemp(*Data));
				}
			}
			// Remove the working payload from our table.
			WorkingPayloads.Remove(Message.Sender);
		}
	}
	else
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Display, "Failed to handle inbound data.");
	}
}

void FLiveLinkOpenTrackIOSource::RemoveAllTransformSubjects(FLiveLinkOpenTrackIOCache* Cache)
{
	// Remove any transform subjects based on this name change.
	for (FName TransformSubjectName : Cache->TransformSubjectNames)
	{
		Client->RemoveSubject_AnyThread({ SourceGuid, TransformSubjectName });
	}
	Cache->TransformSubjectNames.Reset();
}

void FLiveLinkOpenTrackIOSource::ConditionallyPushLiveLinkTransformData(FLiveLinkOpenTrackIOCache* Cache, const FLiveLinkOpenTrackIOData& InData)
{
	const bool bEnableTransformSubjects = IsValid(SavedSourceSettings) ? SavedSourceSettings->ShouldExtractTransformSubjects() : false;
	if (!bEnableTransformSubjects)
	{
		RemoveAllTransformSubjects(Cache);
		return;
	}

	for (const FLiveLinkOpenTrackIOTransform& Transform : InData.Transforms)
	{
		FName TransformName = Cache->GetTransformName(Transform);
		if (!Cache->TransformSubjectNames.Contains(TransformName))
		{
			FLiveLinkStaticDataStruct StaticData(FLiveLinkTransformStaticData::StaticStruct());
			FLiveLinkTransformStaticData& NewStaticData = *StaticData.Cast<FLiveLinkTransformStaticData>();
			NewStaticData.bIsLocationSupported = true;
			NewStaticData.bIsRotationSupported = true;
			NewStaticData.bIsScaleSupported = true;

			Client->PushSubjectStaticData_AnyThread({ SourceGuid, TransformName }, ULiveLinkTransformRole::StaticClass(), MoveTemp(StaticData));
		}

		Cache->TransformSubjectNames.Add(TransformName);
		FLiveLinkFrameDataStruct FrameData(FLiveLinkTransformFrameData::StaticStruct());
		FLiveLinkTransformFrameData& NewFrameData = *FrameData.Cast<FLiveLinkTransformFrameData>();

		NewFrameData.Transform = LiveLinkOpenTrackIOConversions::ToUnrealTransform(Transform);
		Client->PushSubjectFrameData_AnyThread({ SourceGuid, TransformName }, MoveTemp(FrameData));
	}
}

void FLiveLinkOpenTrackIOSource::PushDataToLiveLink_AnyThread(const FLiveLinkOpenTrackIODatagramHeader& Header, FLiveLinkOpenTrackIOData InData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OpenTrackIO::PushDataToLiveLink_AnyThread);

	FLiveLinkOpenTrackIOCache* Cache;

	// Find the right cache for this OpenTrackIO source id and number
	{
		const FString OpenTrackIOStreamKey = FString::Printf(TEXT("%s:%lld"), *InData.SourceId, InData.SourceNumber);

		if (OpenTrackIOCacheMap.Contains(OpenTrackIOStreamKey))
		{
			Cache = OpenTrackIOCacheMap.FindChecked(OpenTrackIOStreamKey).Get();
		}
		else
		{
			Cache = OpenTrackIOCacheMap.Add(OpenTrackIOStreamKey, MakePimpl<FLiveLinkOpenTrackIOCache>()).Get();
		}
	}

	if (!Cache->IsPacketInSequence(Header.SequenceNumber, InData.Timing.SampleRate.GetFrameRate()))
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "Received packet for %ls is out of sequence. Discarding %d.", *InData.SourceId, Header.SequenceNumber);
		return;
	}

	// Update the OpenTrackIO static camera data (make, model, etc.)
	Cache->StaticCamera = InData.Static.Camera;

	// Update the OpenTrackIO static lens data
	if (InData.Static.Lens.IsValid())
	{
		Cache->StaticLens = InData.Static.Lens;
	}

	// Detect if the new OpenTrackIO data changes the Subject Name
	bool bSubjectNameChanged = false;
	{
		const FName SubjectName = Cache->GetSubjectNameFromData(ConnectionSettings.SubjectName, InData);

		if (Cache->SubjectName != SubjectName)
		{
			// Flag this because we'll need to push static data to add the new subject name.
			bSubjectNameChanged = true;

			// Remove the previous Live Link subject, since the name is different.
			if (!Cache->SubjectName.IsNone())
			{
				Client->RemoveSubject_AnyThread({ SourceGuid, Cache->SubjectName });
			}
			RemoveAllTransformSubjects(Cache);

			Cache->SubjectName = SubjectName;
		}
	}

	const bool bShouldApplyTransform = IsValid(SavedSourceSettings) && SavedSourceSettings->ShouldApplyXformToCamera();

	// Update the static data if it has changed or if the subject name was updated.
	{
		FLiveLinkStaticDataStruct StaticData = Cache->MakeStaticData(InData, bShouldApplyTransform);
		const bool bStaticDataChanged = Cache->StaticData != StaticData;

		if (bSubjectNameChanged || bStaticDataChanged)
		{
			Cache->StaticData.InitializeWith(StaticData);
			Client->PushSubjectStaticData_AnyThread({ SourceGuid, Cache->SubjectName }, ULiveLinkOpenTrackIORole::StaticClass(), MoveTemp(StaticData));
		}
	}

	ConditionallyPushLiveLinkTransformData(Cache, InData);

	// Push the per-frame data
	FLiveLinkFrameDataStruct FrameData = Cache->MakeFrameData(InData, bShouldApplyTransform);
	FLiveLinkOpenTrackIOFrameData& OpenTrackIOData = *FrameData.Cast<FLiveLinkOpenTrackIOFrameData>();

	// Note we are moving the InData into the LL data.
	OpenTrackIOData.OpenTrackData = MoveTemp(InData);

	Client->PushSubjectFrameData_AnyThread({ SourceGuid, Cache->SubjectName }, MoveTemp(FrameData));

	Cache->UpdateLastKnownSequenceNumber(Header.SequenceNumber);
}

void FLiveLinkOpenTrackIOSource::StopUdpReceivers()
{
	if (MulticastReceiver)
	{
		MulticastReceiver->Stop();
	}

	if (UnicastReceiver)
	{
		UnicastReceiver->Stop();
	}
}

void FLiveLinkOpenTrackIOSource::CloseSockets()
{
	StopUdpReceivers();

	// Close sockets to unblock receiver threads.

	if (MulticastSocket)
	{
		MulticastSocket->Close();
	}

	if (UnicastSocket)
	{
		UnicastSocket->Close();
	}

	// Join receiver threads. Safe because sockets are closed.
	MulticastReceiver.Reset();
	UnicastReceiver.Reset();

	// Destroy socket objects
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	check(SocketSubsystem);

	if (MulticastSocket)
	{
		SocketSubsystem->DestroySocket(MulticastSocket);
		MulticastSocket = nullptr;
	}
	if (UnicastSocket)
	{
		SocketSubsystem->DestroySocket(UnicastSocket);
		UnicastSocket = nullptr;
	}

	// Safe to clear because UDP threads are joined above.
	OpenTrackIOCacheMap.Reset();
}

bool FLiveLinkOpenTrackIOSource::OpenMulticastSocket()
{
	// Create an IPv4 TCP Socket
	const uint32 ReceiveBufferSize = 2 * 1024 * 1024;

	/** Create a multicast socket. */
	MulticastSocket = FUdpSocketBuilder(TEXT("UdpOpenTrackIO_MulticastSocket"))
						  .AsNonBlocking()
						  .AsReusable()
#if PLATFORM_WINDOWS

						  /**
						   * If multiple bus instances bind the same unicast ip:port combination (allowed as the socket is marked as reusable), then for each packet
						   * sent to that ip:port combination, only one of the instances (at the discretion of the OS) will receive it. The instance that receives the
						   * packet may vary over time, seemingly based on the congestion of its socket. This isn't the intended usage.
						   *
						   * To allow traffic to be sent directly to unicast for discovery, set the interface and port for the unicast endpoint
						   * However for legacy reason, keep binding this as well, although it might be unreliable in some cases
						   */
						  .BoundToAddress(UnicastEndpoint.Address)
#endif
						  .BoundToPort(MulticastEndpoint.Port)
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
						  .WithMulticastLoopback()
						  .WithMulticastInterface(UnicastEndpoint.Address)
#endif
						  .WithReceiveBufferSize(ReceiveBufferSize);

	if (!MulticastSocket)
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Warning, "StartTransport failed to create multicast socket on %ls, joined to %ls", *UnicastEndpoint.ToString(), *MulticastEndpoint.ToString());
		return false;
	}

	UE::LiveLinkOpenTrackIO::Private::JoinedToGroup(UnicastEndpoint, MulticastEndpoint, MulticastSocket);

	// Create receiver for socket above

	const FTimespan ThreadWaitTime = FTimespan::FromMilliseconds(100);

	MulticastReceiver = MakeUnique<FUdpSocketReceiver>(MulticastSocket, ThreadWaitTime, TEXT("LiveLinkOpenTransportIO_MulticastReceiver"));
	MulticastReceiver->OnDataReceived().BindRaw(this, &FLiveLinkOpenTrackIOSource::HandleInboundData);
	MulticastReceiver->SetMaxReadBufferSize(2048);
	MulticastReceiver->SetThreadStackSize(512 * 1024);
	MulticastReceiver->Start();

	return true;
}

bool FLiveLinkOpenTrackIOSource::OpenUnicastSocket()
{
	// Create an IPv4 TCP Socket
	const uint32 ReceiveBufferSize = 2 * 1024 * 1024;

	UnicastSocket = FUdpSocketBuilder(TEXT("UdpOpenTrackIO_UnicastSocket"))
						.AsNonBlocking()
						.BoundToEndpoint(UnicastEndpoint)
						.WithMulticastLoopback()
						.WithReceiveBufferSize(ReceiveBufferSize);

	if (!UnicastSocket)
	{
		UE_LOGF(LogLiveLinkOpenTrackIO, Error, "Failed to create OpenTrackIO socket on %ls", *UnicastEndpoint.ToString());
		return false;
	}

	const int32 PortNo = UnicastSocket->GetPortNo();
	FString IpPortStr = FString::Format(TEXT("{0}:{1}"), { UnicastEndpoint.Address.ToString(), PortNo });
	UE_LOGF(LogLiveLinkOpenTrackIO, Display, "OpenTrackIO unicast socket socket bound to '%ls'.", *IpPortStr);

	// Create receiver for socket above

	const FTimespan ThreadWaitTime = FTimespan::FromMilliseconds(100);

	UnicastReceiver = MakeUnique<FUdpSocketReceiver>(UnicastSocket, ThreadWaitTime, TEXT("LiveLinkOpenTransportIO_UnicastReceiver"));
	UnicastReceiver->OnDataReceived().BindRaw(this, &FLiveLinkOpenTrackIOSource::HandleInboundData);
	UnicastReceiver->SetMaxReadBufferSize(2048);
	UnicastReceiver->SetThreadStackSize(512 * 1024);
	UnicastReceiver->Start();

	return true;
}

bool FLiveLinkOpenTrackIOSource::OpenSockets()
{
	switch (SavedSourceSettings->Protocol)
	{
		case ELiveLinkOpenTrackIONetworkProtocol::Unicast:
		{
			if (!OpenUnicastSocket())
			{
				return false;
			}

			break;
		}
		case ELiveLinkOpenTrackIONetworkProtocol::Multicast:
		{
			if (!OpenMulticastSocket())
			{
				return false;
			}

			break;
		}
		default:
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
