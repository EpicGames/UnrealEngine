// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "LiveLinkOpenTrackIOConnectionSettings.h"
#include "LiveLinkOpenTrackIOParser.h"
#include "LiveLinkOpenTrackIOSourceSettings.h"

#include "Common/UdpSocketReceiver.h"
#include "Containers/MpscQueue.h"
#include "Templates/UniquePtr.h"
#include "Templates/PimplPtr.h"

#include "Delegates/IDelegateInstance.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/Runnable.h"

struct ULiveLinkOpenTrackIOSettings;

class IOpenTrackIO;
class ILiveLinkClient;
class FArrayReader;
enum class ESPMode : uint8;
template<class ObjectType, ESPMode InMode> class TSharedPtr;

struct FLiveLinkOpenTrackIOData;
struct FLiveLinkOpenTrackIODatagramHeader;
struct FLiveLinkOpenTrackIOCache;

enum class ELiveLinkOpenTrackIOState : uint8
{
	/** Default state on source initialization. */
	NotStarted = 0,

	/** We have parsed the endpoints data from the source settings and will prepare to open receiving connections */
	EndpointsReady,

	/** Sockets have been opened and we are listening for data. */
	Receiving,

	/** The user has changed endpoints or user requested a socket reset. */
	ResetRequested,

	/** Final state */
	ShutDown,
};

class LIVELINKOPENTRACKIO_API FLiveLinkOpenTrackIOSource : public ILiveLinkSource, public TSharedFromThis<FLiveLinkOpenTrackIOSource>
{
public:

	FLiveLinkOpenTrackIOSource(FLiveLinkOpenTrackIOConnectionSettings ConnectionSettings);

	virtual ~FLiveLinkOpenTrackIOSource();

	// Begin ILiveLinkSource Interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;

	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual void Update() override;
	
	virtual FText GetSourceType() const override { return SourceType; };
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override { return SourceStatus; }

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override { return ULiveLinkOpenTrackIOSourceSettings::StaticClass(); }
	// End ILiveLinkSource Interface

private:

	/** Create listening sockets for OpenTrack channels. Returns true if socket handles were successfully opened and receivers started. */
	bool OpenSockets();

	/** Create Multicast socket (called by OpenSockets) */
	bool OpenMulticastSocket();

	/** Create Uniicast socket (called by OpenSockets) */
	bool OpenUnicastSocket();

	/** Close any open sockets. */
	void CloseSockets();

	/** Stop the udp receivers, but does not destroy the sockets. */
	void StopUdpReceivers();

	/** Read the source settings and assign the endpoint addresses. Returns true if the endpoints are ready */
	bool ParseEndpointFromSourceSettings();

	/** Delegate for handling inbound segments. */
	void HandleInboundData(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& InData, const FIPv4Endpoint& InSender);

	/** Copy the open track data into Live Link equivalent data. This will establish any static data if the InData specifies it. */
	void PushDataToLiveLink_AnyThread(const FLiveLinkOpenTrackIODatagramHeader& Header, FLiveLinkOpenTrackIOData InData);

	/** Remove all transform subjects from the LL source. */
	void RemoveAllTransformSubjects(FLiveLinkOpenTrackIOCache* Cache);

	/** Push transform data to Live Link if the user has asked for it in the SavedSourceSettings. New subjects will get created automatically. */
	void ConditionallyPushLiveLinkTransformData(FLiveLinkOpenTrackIOCache* Cache, const FLiveLinkOpenTrackIOData& InData);
	
	/** Sets connection state */
	inline void SetConnectionState(const ELiveLinkOpenTrackIOState InConnectionState)
	{
		checkSlow(IsInGameThread());
		ConnectionState = InConnectionState;
	}

	/** Called when FCoreDelegates::OnEnginePreExit fires */
	void OnEnginePreExit();

private:

	struct FInboundMessage
	{
		/** Holds the segment data. */
		TSharedPtr<FArrayReader, ESPMode::ThreadSafe> Data;

		/** Holds the sender's network endpoint. */
		FIPv4Endpoint Sender;
	};

	/** Pointer to implementation of our OpenTrackIO cache so that we can track camera and lens information. */
	using FOpenTrackIOCachePtr = TPimplPtr<FLiveLinkOpenTrackIOCache>;

	/** Keep track of payloads per endpoint. This is used to keep track of segmented data per endpoint.  */
	TMap<FIPv4Endpoint, FOpenTrackIOHeaderWithPayload> WorkingPayloads;

	/** Cache of data relevant to open track protocol. It is keyed by the sourceId. We can support multiple source ids */
	TMap<FString, FOpenTrackIOCachePtr> OpenTrackIOCacheMap;
	
	/** The LiveLink client provided via ReceiveClient*/
	ILiveLinkClient* Client = nullptr;

	/** Our identifier in LiveLink */
	FGuid SourceGuid;

	/** Text objects that are reflected back in the Live Link UI. */
	FText SourceType;
	FText SourceMachineName;
	FText SourceStatus;

	/** Handle the source settings for this source. The IP addresses are stored so that they can be changed after source creation. */
	TObjectPtr<ULiveLinkOpenTrackIOSourceSettings> SavedSourceSettings;

	/** Holds the connection settings to use for this source. This is established when the source is first created. */
	FLiveLinkOpenTrackIOConnectionSettings ConnectionSettings;

	/** Holds the multicast socket. */
	FSocket* MulticastSocket = nullptr;

	/** Holds the unicast socket receiver. */
	TUniquePtr<FUdpSocketReceiver> UnicastReceiver;
	
	/** Holds the multicast socket receiver. */
	TUniquePtr<FUdpSocketReceiver> MulticastReceiver;

	/** Holds the unicast socket. */
	FSocket* UnicastSocket = nullptr;

	/** Holds the local endpoint to receive messages. */
	FIPv4Endpoint UnicastEndpoint;

	/** Multicast Endpoint endpoint to receive messages. */
	FIPv4Endpoint MulticastEndpoint;

	/** State of the state machine. Only write/read in the game thread */
	ELiveLinkOpenTrackIOState ConnectionState = ELiveLinkOpenTrackIOState::NotStarted;

	/** Cached sender ip address, used to update the UI */
	FIPv4Endpoint LastSender_RunnableThread;

	/** Used to detect staleness in the source */
	std::atomic<double> LastDataReadTime = 0;

	/** Tells the state machine that a shutdown of the source has been requested. One time only use, will never get cleared. */
	bool bShutdownRequested = false;

	/** Tells the state machine that the connection should be reset, probably because a relevant setting has been changed. */
	bool bResetRequested = false;
};
