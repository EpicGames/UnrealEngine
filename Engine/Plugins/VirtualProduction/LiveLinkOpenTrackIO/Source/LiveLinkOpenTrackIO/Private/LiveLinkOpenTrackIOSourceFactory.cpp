// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenTrackIOSourceFactory.h"

#include "ILiveLinkClient.h"
#include "LiveLinkOpenTrackIOSource.h"
#include "LiveLinkOpenTrackIOSourceSettings.h"

#include "SLiveLinkOpenTrackIOSourceFactory.h"

#define LOCTEXT_NAMESPACE "LiveLinkOpenTrackIOSourceFactory"

FText ULiveLinkOpenTrackIOSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "Live Link OpenTrackIO Source");
}

FText ULiveLinkOpenTrackIOSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Creates a Live Link OpenTrackIO source (https://www.opentrackio.org/).");
}

TSharedPtr<SWidget> ULiveLinkOpenTrackIOSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkOpenTrackIOSourceFactory)
		.OnConnectionSettingsAccepted(FOnLiveLinkOpenTrackIOConnectionSettingsAccepted::CreateUObject(this, &ULiveLinkOpenTrackIOSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated));
}

TSharedPtr<ILiveLinkSource> ULiveLinkOpenTrackIOSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkOpenTrackIOConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkOpenTrackIOConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("ULiveLinkOpenTrackIOSourceFactory"));
	}
	return MakeShared<FLiveLinkOpenTrackIOSource>(ConnectionSettings);
}

void ULiveLinkOpenTrackIOSourceFactory::CreateSourceFromSettings(FLiveLinkOpenTrackIOConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FLiveLinkOpenTrackIOConnectionSettings::StaticStruct()->ExportText(ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr);

	TSharedPtr<FLiveLinkOpenTrackIOSource> SharedPtr = MakeShared<FLiveLinkOpenTrackIOSource>(InConnectionSettings);
	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}

#undef LOCTEXT_NAMESPACE
