// Copyright Epic Games, Inc. All Rights Reserved.
//
#include "LiveLinkOpenTrackIOTranscoder.h"

#include "Containers/UnrealString.h"
#include "LiveLinkOpenTrackIOConnectionSettings.h"
#include "LiveLinkOpenTrackIOConversions.h"
#include "LiveLinkOpenTrackIOLiveLinkTypes.h"

#include "Misc/App.h"
#include "Misc/QualifiedFrameTime.h"
#include "Roles/LiveLinkCameraTypes.h"

bool FLiveLinkOpenTrackIOCache::IsPacketInSequence(uint16 SequenceNumber, const FFrameRate& InRate) const
{
	// Initially the expected sequence number is -1, which indicates we have not received our first packet yet.
	if (ExpectedSequenceNumber > 0 && SequenceNumber < ExpectedSequenceNumber)
	{
		// Ensure a minimum rate. Also avoids divide by zero if the external producer sends zero rate.
		constexpr double MinimumPeriod = 1.0 / 120.0;
		const double Period = static_cast<double>(InRate.Denominator) / FMath::Max(1, InRate.Numerator);
		const double SampleRateExpiry = 3 * FMath::Max(MinimumPeriod, Period);

		// We we have exceeded a timeout period then consider the sequence stream as being reset.
		// In other words, sequence numbers are sequential only.
		if ((FPlatformTime::Seconds() - LastDataReceiveTimeInSeconds) < SampleRateExpiry)
		{
			return false;
		}
	}

	return true;
}

FName FLiveLinkOpenTrackIOCache::GetSubjectNameFromData(const FString& InSubjectName, const FLiveLinkOpenTrackIOData& InData)
{
	// If no InSubjectName hint is provided, fallback names will be automatically generated based on the data.

	if (!InSubjectName.IsEmpty() && (InSubjectName != FLiveLinkOpenTrackIOConnectionSettings::AutoSubjectName))
	{
		return FName(InSubjectName);
	}

	// First fallback: Static camera
	if (StaticCamera.IsValid())
	{
		const FString Postfix = FString::Printf(TEXT("%03lld"), InData.SourceNumber);
		return UE::OpenTrackIO::ConvertTypeToFName(StaticCamera, Postfix);
	}

	// Next fallback: Static lens
	if (StaticLens.IsSet())
	{
		const FString Postfix = FString::Printf(TEXT("%03lld"), InData.SourceNumber);
		return UE::OpenTrackIO::ConvertTypeToFName(StaticLens.GetValue(), Postfix);
	}

	// Last fallback: Use the SourceId and Source Number of this OTrIO stream.

	constexpr int32 NumLastSourceIdCharsToTake = 6;
	const FString LastOfSourceId = InData.SourceId.Right(FMath::Min(NumLastSourceIdCharsToTake, InData.SourceId.Len()));

	return FName(FString::Printf(TEXT("OTrIO_%s_%03lld"), *LastOfSourceId, InData.SourceNumber));
}

FName FLiveLinkOpenTrackIOCache::GetTransformName(const FLiveLinkOpenTrackIOTransform& InTransform)
{
	if (!SubjectName.IsNone())
	{
		return FName(FString::Printf(TEXT("%s_%s"), *SubjectName.ToString(), *InTransform.Id));
	}

	return FName(InTransform.Id);
}

FLiveLinkStaticDataStruct FLiveLinkOpenTrackIOCache::MakeStaticData(const FLiveLinkOpenTrackIOData& Data, bool bApplyXform)
{
	/** Lens class inherits from the camera data so we use that as our base for the static struct. */

	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkOpenTrackIOStaticData::StaticStruct());
	FLiveLinkOpenTrackIOStaticData& NewStaticData = *StaticDataStruct.Cast<FLiveLinkOpenTrackIOStaticData>();

	if (Data.Lens.PinholeFocalLength.IsSet())
	{
		NewStaticData.bIsFocalLengthSupported = true;
	}

	if (Data.Lens.FStop.IsSet())
	{
		NewStaticData.bIsApertureSupported = true;
	}

	if (Data.Lens.FocusDistance.IsSet())
	{
		NewStaticData.bIsFocusDistanceSupported = true;
	}

	if ((bApplyXform && Data.Transforms.Num() > 0))
	{
		NewStaticData.bIsLocationSupported = true;
		NewStaticData.bIsRotationSupported = true;
		NewStaticData.bIsScaleSupported = true;
	}

	// For data that depends on OpenTrackIO static data, if this particular packet
	// does not contain static data, then we check the cache.
	{
		// In OpenTrackIO, we opt for sending the filmback in the frame data instead of static data,
		// because some cameras with built-in undistortion may update this value dynamically.

		if ((Data.Static.Camera.ActiveSensorPhysicalDimensions.Height.IsSet()
				&& Data.Static.Camera.ActiveSensorPhysicalDimensions.Width.IsSet()))
		{
			NewStaticData.bIsDynamicFilmbackSupported = true;
		}
		else
		{
			if (StaticCamera.ActiveSensorPhysicalDimensions.Height.IsSet()
				&& StaticCamera.ActiveSensorPhysicalDimensions.Width.IsSet())
			{
				NewStaticData.bIsDynamicFilmbackSupported = true;
			}
		}
	}

	// Lens Distortion Model

	if (Data.Lens.Distortion.Num() > 0)
	{
		// For now we will pick the first model in the array.
		const FLiveLinkOpenTrackIOLens_DistortionCoeff& Distortion = Data.Lens.Distortion[0];
		if (Distortion.Model.IsNone())
		{
			// OpentrackIO's default model is "Brown-Conrady D-U".
			NewStaticData.LensModel = UE::OpenTrackIO::BrownConradyDU;
		}
		else
		{
			// Use the model name directly because Unreal comes with Lens Models named the same as OpenTrackIO.
			// If that ever changed, we'd need a name mapping function here.
			NewStaticData.LensModel = Distortion.Model;
		}
	}

	return StaticDataStruct;
}

FLiveLinkFrameDataStruct FLiveLinkOpenTrackIOCache::MakeFrameData(const FLiveLinkOpenTrackIOData& Data, bool bApplyXform)
{
	using namespace LiveLinkOpenTrackIOConversions;

	FTransform FinalTransform;
	if (bApplyXform)
	{
		/** Compute the final transform by concatentating the transforms in the frame data. */
		for (const FLiveLinkOpenTrackIOTransform& Transform : Data.Transforms)
		{
			FinalTransform = ToUnrealTransform(Transform) * FinalTransform;
		}
	}

	// Apply entrance pupil offset as a final transform in the forward direction.
	const FLiveLinkOpenTrackIO_XYZ EntrancePupilAsXYZ(0, Data.Lens.EntrancePupilOffset, 0);

	const FTransform PupilOffset(
		FQuat::Identity,
		ToUnrealTranslation(EntrancePupilAsXYZ),
		FVector(1, 1, 1));

	FinalTransform = PupilOffset * FinalTransform;

	FQualifiedFrameTime SampleTC;
	if (Data.Timing.IsDefault())
	{
		const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
		if (CurrentFrameTime)
		{
			SampleTC = *CurrentFrameTime;
		}
	}
	else
	{
		SampleTC = Data.Timing.Timecode.GetQualifiedFrameTime();
	}

	const FLiveLinkOpenTrackIOStaticCamera* Camera = &StaticCamera;
	const FLiveLinkOpenTrackIOStaticLens* Lens = StaticLens.IsSet() ? &StaticLens.GetValue() : nullptr;

	FLiveLinkFrameDataStruct FrameDataStruct = FLiveLinkFrameDataStruct(FLiveLinkOpenTrackIOFrameData::StaticStruct());
	FLiveLinkOpenTrackIOFrameData& CameraAndLensData = *FrameDataStruct.Cast<FLiveLinkOpenTrackIOFrameData>();

	ToUnrealLens(CameraAndLensData, &Data.Lens, Camera);

	CameraAndLensData.Transform = FinalTransform;
	CameraAndLensData.MetaData.SceneTime = SampleTC;
	CameraAndLensData.WorldTime = FPlatformTime::Seconds();

	// Add any custom meta data.
	for (const FLiveLinkOpenTrackIOCustomDataField& Field : Data.Custom.LiveLinkMetaData)
	{
		CameraAndLensData.MetaData.StringMetaData.Add(FName(Field.Key), Field.Value);
	}

	return FrameDataStruct;
}
