// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkOpenTrackIOTypes.h"

#include "LiveLinkTypes.h"

#include "Containers/Set.h"
#include "Misc/Optional.h"

struct FLiveLinkOpenTrackIOCache
{
	/** Last known Subject Name */
	FName SubjectName;

	/** List of transform subject names known. */
	TSet<FName> TransformSubjectNames;

	/** Live Link Static Data, used to detect changes */
	FLiveLinkStaticDataStruct StaticData;

	/** OpenTrackIO static camera data (make, model, etc.) */
	FLiveLinkOpenTrackIOStaticCamera StaticCamera;

	/** OpenTrackIO static lens data */
	TOptional<FLiveLinkOpenTrackIOStaticLens> StaticLens;

	/** Cached state of exposing the transforms as subjects. */
	bool bSendTransformsAsSubjects = false;

	/**
	 * Check the given sequence number and return true if it is the expected number. Sequence numbers should be incrementing
	 * modulo UINT16_MAX.  This also accounts for delays in transmission based on sample rate provided by the first inbound packets.
	 */
	bool IsPacketInSequence(uint16 SequenceNumber, const FFrameRate& InRate) const;

	/**
	 * Caches the processed sequence number so that it can be compared when the next packet is received.
	 */
	void UpdateLastKnownSequenceNumber(uint16 SequenceNumber)
	{
		ExpectedSequenceNumber = SequenceNumber + 1;
		if (ExpectedSequenceNumber > UINT16_MAX)
		{
			ExpectedSequenceNumber = 0;
		}
		LastDataReceiveTimeInSeconds = FPlatformTime::Seconds();
	}

	/** Constructs a subject name from the given data. */
	FName GetSubjectNameFromData(const FString& Prefix, const FLiveLinkOpenTrackIOData& Data);

	/** Constructs a subject name from the given data. */
	FName GetTransformName(const FLiveLinkOpenTrackIOTransform& InTransform);

	/** Create Live Link static data from the OpenTrackIO data */
	FLiveLinkStaticDataStruct MakeStaticData(const FLiveLinkOpenTrackIOData& StaticData, const bool bApplyXform);

	/** Create Live Link the per-frame data from the OpenTrackIO data */
	FLiveLinkFrameDataStruct MakeFrameData(const FLiveLinkOpenTrackIOData& Data, const bool bApplyXform);

private:
	/** Tracks the current sequence number for the source. */
	int32 ExpectedSequenceNumber = -1;

	/** Store the last time in FPlatformTime::Seconds() we received data from this source. */
	double LastDataReceiveTimeInSeconds = 0;
};
