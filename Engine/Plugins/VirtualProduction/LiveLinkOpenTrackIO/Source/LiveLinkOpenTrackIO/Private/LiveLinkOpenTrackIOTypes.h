// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/QualifiedFrameTime.h"
#include "UObject/ObjectMacros.h"
#include "Containers/Array.h"

#include "Math/Matrix.h"
#include "Math/Transform.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "LiveLinkOpenTrackIOTypes.generated.h"


/**
 * Contains a float and bIsSet flag, mimicking TOptional, to be exposed in Blueprints.
 * The intention is that it is clear to the user if the value contained has been set
 * by the sender or not.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Optional Float (OpenTrackIO)"))
struct FOpenTrackIOOptionalFloat
{
	GENERATED_BODY()

	/** Whether Value has been set or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional")
	bool bIsSet = false;

	/** The actual float value (only valid when bIsSet == true) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional", meta = (EditCondition = "bIsSet"))
	float Value = 0;

public:

	/** Equals Operator */
	bool operator==(const FOpenTrackIOOptionalFloat& Other) const
	{
		if (bIsSet != Other.bIsSet)
		{
			return false;
		}

		// If neither is set, we consider them equal even if the Value is different.
		if (!bIsSet)
		{
			return true;
		}

		// both are set, we compare the Value
		return FMath::IsNearlyEqual(Value, Other.Value);
	}

	/** Different Than operator */
	bool operator!=(const FOpenTrackIOOptionalFloat& Other) const
	{
		return !(*this == Other);
	}

	/** Check if Value has been set. Mimicks IsSet() in TOptional */
	bool IsSet() const
	{
		return bIsSet;
	}

	/** Gets the set Value. Mimicks GetValue() in TOptional */
	float GetValue() const
	{
		return Value;
	}

	/** Sets the Value of this container */
	void SetValue(float InValue)
	{
		bIsSet = true;
		Value = InValue;
	}

	/** Clears the bIsSet flag. Mimicks Reset() in TOptional */
	void Reset()
	{
		bIsSet = false;
		Value = 0.0f;
	}
};


/**
 * Contains an int32 and bIsSet flag, mimicking TOptional, to be exposed in Blueprints.
 * The intention is that it is clear to the user if the value contained has been set
 * by the sender or not.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Optional Int32 (OpenTrackIO)"))
struct FOpenTrackIOOptionalInt32
{
	GENERATED_BODY()

	/** Whether Value has been set or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional")
	bool bIsSet = false;

	/** The actual int32 value (only valid when bIsSet == true) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional", meta = (EditCondition = "bIsSet"))
	int32 Value = 0;

public:

	/** Equals Operator */
	bool operator==(const FOpenTrackIOOptionalInt32& Other) const
	{
		if (bIsSet != Other.bIsSet)
		{
			return false;
		}

		// If neither is set, we consider them equal even if the Value is different.
		if (!bIsSet)
		{
			return true;
		}

		// both are set, we compare the Value
		return Value == Other.Value;
	}

	/** Different Than operator */
	bool operator!=(const FOpenTrackIOOptionalInt32& Other) const
	{
		return !(*this == Other);
	}

	/** Check if Value has been set. Mimicks IsSet() in TOptional */
	bool IsSet() const
	{
		return bIsSet;
	}

	/** Gets the set Value. Mimicks GetValue() in TOptional */
	int32 GetValue() const
	{
		return Value;
	}

	/** Sets the Value of this container */
	void SetValue(int32 InValue)
	{
		bIsSet = true;
		Value = InValue;
	}

	/** Clears the bIsSet flag. Mimicks Reset() in TOptional */
	void Reset()
	{
		bIsSet = false;
		Value = 0;
	}
};


/**
 * Contains a double and bIsSet flag, mimicking TOptional, to be exposed in Blueprints.
 * The intention is that it is clear to the user if the value contained has been set
 * by the sender or not.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Optional Double (OpenTrackIO)"))
struct FOpenTrackIOOptionalDouble
{
	GENERATED_BODY()

	/** Whether Value has been set or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional")
	bool bIsSet = false;

	/** The actual double value (only valid when bIsSet == true) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Optional", meta = (EditCondition = "bIsSet"))
	double Value = 0.0;

public:

	/** Equals Operator */
	bool operator==(const FOpenTrackIOOptionalDouble& Other) const
	{
		if (bIsSet != Other.bIsSet)
		{
			return false;
		}

		// If neither is set, we consider them equal even if the Value is different.
		if (!bIsSet)
		{
			return true;
		}

		// both are set, we compare the Value
		return FMath::IsNearlyEqual(Value, Other.Value, UE_DOUBLE_SMALL_NUMBER);
	}

	/** Different Than operator */
	bool operator!=(const FOpenTrackIOOptionalDouble& Other) const
	{
		return !(*this == Other);
	}

	/** Check if Value has been set. Mimicks IsSet() in TOptional */
	bool IsSet() const
	{
		return bIsSet;
	}

	/** Gets the set Value. Mimicks GetValue() in TOptional */
	double GetValue() const
	{
		return Value;
	}

	/** Sets the Value of this container */
	void SetValue(double InValue)
	{
		bIsSet = true;
		Value = InValue;
	}

	/** Clears the bIsSet flag. Mimicks Reset() in TOptional */
	void Reset()
	{
		bIsSet = false;
		Value = 0.0;
	}
};


/**
 * Represents frame rate type from OpenTrack I/O. Value is represented in hertz using an fractional representation
 * Numerator / Denominator. The default value is 24 / 1 or 24 cycles per second.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOFrameRate
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOFrameRate DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOFrameRate&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOFrameRate&) const = default;

	FFrameRate GetFrameRate() const
	{
		return FFrameRate(static_cast<uint32>(Num), static_cast<uint32>(Denom));
	}

	/** Numerator of the frame rate value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Num = 24;

	/** Denominator of the frame rate value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Denom = 1;
};

/** Concepts to test for the existance of member variables on our structs. */
template<typename T>
concept CHasMakeMember = requires(T t)
{ t.Make; };

template<typename T>
concept CHasModelMember = requires(T t)
{ t.Model; };

template<typename T>
concept CHasLabelMember = requires(T t)
{ t.Label; };

template <typename T>
concept CHasModelAndMake = CHasMakeMember<T> && CHasModelMember<T>;

namespace UE::OpenTrackIO
{

template<CHasModelAndMake T>
FName ConvertTypeToFName(const T& Type, const FString& Postfix)
{
	FNameBuilder Builder;	
	if constexpr(CHasLabelMember<T>)
	{
		Builder.Append(*Type.Label);
	}
	else
	{
		Builder.Appendf(TEXT("%s_%s"), *Type.Make, *Type.Model);
	}
	
	if (!Postfix.IsEmpty())
	{
		Builder.Appendf(TEXT(":%s"), *Postfix);
	}
	
	return *Builder;
}

}
/**
 * Height and width of the active area of the camera sensor in millimeters
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOSensorDimensions
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOSensorDimensions DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOSensorDimensions&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOSensorDimensions&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat Height;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat Width;
};

/**
 *  Photosite resolution of the active area of the camera sensor in pixels
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOSensorResolution
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOSensorResolution DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOSensorResolution&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOSensorResolution&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Height = 1920;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Width = 1080;
};

/**
 * Nominal ratio of height to width of the image of an axis-aligned square captured by the camera sensor.
 * It can be used to de-squeeze images but is not however an exact number over the entire captured
 * area due to a lens’ intrinsic analog nature.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOAnamorphicSqueeze
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOAnamorphicSqueeze DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOAnamorphicSqueeze&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOAnamorphicSqueeze&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Num = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Denom = 1;		
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStaticCamera
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOStaticCamera DefaultValue;
		return DefaultValue == *this;
	}

	bool operator==(const FLiveLinkOpenTrackIOStaticCamera& Other) const
	{
		return Other.Make == Make
			&& Other.Model == Model
			&& Other.SerialNumber == SerialNumber
			&& Other.Label == Label
			&& Other.FdlLink == FdlLink
			&& Other.FirmwareVersion == FirmwareVersion; 
	}
	bool operator!=(const FLiveLinkOpenTrackIOStaticCamera& Other) const
	{
		return !(Other == *this);
	}
	
	/** If the provided static camera specifies the correct properties to be considered valid. */
	bool IsValid() const
	{
		return !Make.IsEmpty() && !Model.IsEmpty() && !SerialNumber.IsEmpty() && !Label.IsEmpty();
	}

	void UpdateStaticData(const FLiveLinkOpenTrackIOStaticCamera& Other)
	{
		ActiveSensorPhysicalDimensions = Other.ActiveSensorPhysicalDimensions;
		IsoSpeed = Other.IsoSpeed;
		ShutterAngle = Other.ShutterAngle;
	}
	
	/** Object representing the capture rate.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOFrameRate CaptureFrameRate;

	/** Object representing the sensor dimensions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOSensorDimensions ActiveSensorPhysicalDimensions;

	/** Object representing the sensor resolution.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOSensorResolution ActiveSensorResolution;

	/** Non-blank string naming camera manufacturer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Make;

	/** Non-blank string naming camera model. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Model;

	/** Non-blank string for camera serial number. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString SerialNumber;

	/** Non-blank string identifying camera firmware version */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString FirmwareVersion;

	/** Non-blank string containing user-determined camera identifier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Label;

	/** Anamorphic squeeze ratio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOAnamorphicSqueeze AnamorphicSqueeze;

	/** Arithmetic ISO scale as defined in ISO 12232 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 IsoSpeed = 400;

	/**
	 * URN identifying the ASC Framing Decision List used by the camera.
	 * Pattern:  ^urn:uuid:[0-9a-f]{8}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{12}$
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString FdlLink;

	/**
	 * Shutter speed as a fraction of the capture frame rate. The shutter speed (in units of 1/s) is equal
	 * to the value of the parameter divided by 360 times the capture frame rate.
	 */
	UPROPERTY(meta = (ClampMin = "0.0", ClampMax = "360.0"))
	float ShutterAngle = 180.0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStaticLens
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOStaticLens DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOStaticLens&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOStaticLens&) const = default;

	/** If the provided static lens specifies the correct properties to be considered valid. */
	bool IsValid() const
	{
		return !Make.IsEmpty() && !Model.IsEmpty() && !SerialNumber.IsEmpty();
	}

	void UpdateStaticData(const FLiveLinkOpenTrackIOStaticLens&)
	{
		// TODO No-op for now. 
	}
	
	/**
	 * Static maximum overscan factor on lens distortion. This is an alternative to
	 * providing dynamic overscan values each frame. Note it should be the maximum
	 * of both projection-matrix-based and field-of-view-based rendering as per the OpenLensIO documentation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float DistortionOverscanMax = 1.0;

	/**
	 * Static maximum overscan factor on lens undistortion. This is an alternative
	 * to providing dynamic overscan values each frame. Note it should bethe maximum of both
	 * projection-matrix-based and field-of-view-based rendering as per the OpenLensIO
	 * documentation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float UndistortionOverscanMax = 1.0;

	/** Non-blank string naming lens manufacturer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Make;

	/** Non-blank string identifying lens model */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Model;

	/** Non-blank string uniquely identifying the lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString SerialNumber;

	/** Non-blank string identifying lens firmware version */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString FirmwareVersion;

	/** Nominal focal length of the lens. The number printed on the side of a prime lens, e.g. 50 mm, and undefined in the case of a zoom lens. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float NominalFocalLength = 50;

	/** List of free strings that describe the history of calibrations of the lens. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<FString> CalibrationHistory;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStaticTracker
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOStaticTracker DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOStaticTracker&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOStaticTracker&) const = default;

	/** If the provided static lens specifies the correct properties to be considered valid. */
	bool IsValid() const
	{
		return !Make.IsEmpty() && !Model.IsEmpty() && !SerialNumber.IsEmpty();
	}

	/** Non-blank string naming tracking device manufacturer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Make;

	/**  Non-blank string identifying tracking device model */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Model;

	/** Non-blank string uniquely identifying the tracking device */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString SerialNumber;

	/**  Non-blank string identifying tracking device firmware version */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString FirmwareVersion;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStaticDuration
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOStaticDuration DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOStaticDuration&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOStaticDuration&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Num = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Denom = 1; 
};

/**
 * Static data from a parsed OpenTrackIO datagram. Per the spec this will be sent periodically from the source.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOStatics
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOStatics DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOStatics&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOStatics&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOStaticDuration Duration;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOStaticCamera Camera;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOStaticLens Lens;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOStaticTracker Tracker;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOTracker
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOTracker DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOTracker&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOTracker&) const = default;
	
	/** Non-blank string containing notes about tracking system */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Notes;

	/** Boolean indicating whether tracking system is recording data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	bool Recording = false;

	/** Non-blank string describing the recording slate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Slate;

	/** Non-blank string describing status of tracking system */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Status;
};

/** 
 * Representation of a OpenTrackIO Timestamp.
 *
 * The timestamp comprises a 48-bit unsigned integer (seconds), a 32-bit unsigned integer (nanoseconds).
 *
 * We use 64-bit unsigned values for storing seconds. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOTimestamp
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOTimestamp DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOTimestamp&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOTimestamp&) const = default;
	
	/** Although the schema specifies a 48 bit number. We allocate a full 64 bit integer here to store it. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int64 Seconds = 0;

	/** A 32-bit unsigned number for nanoseconds */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Nanoseconds = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOSynchronizationOffsets
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOSynchronizationOffsets DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOSynchronizationOffsets&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOSynchronizationOffsets&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Translation = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Rotation = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double LensEncoders = 0.0f;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIO_PTPLeaderPriorities
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIO_PTPLeaderPriorities DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIO_PTPLeaderPriorities&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIO_PTPLeaderPriorities&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	uint8 Priority1 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	uint8 Priority2 = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIO_PTP
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIO_PTP DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIO_PTP&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIO_PTP&) const = default;
	
	/** PTP Profile: Can be one the following values [IEEE Std 1588-2019, IEEE Std 802.1AS-2020, SMPTE ST2059-2:2021] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Profile = TEXT("SMPTE ST2059-2:2021");

	UPROPERTY(meta = (ClampMin = "0"))
	int8 Domain = 0;

	/**
	 * PTP Leader identity follows the pattern:
	 * 
	 * (?:^[0-9a-f]{2}(?::[0-9a-f]{2}){5}$)|(?:^[0-9a-f]{2}(?:-[0-9a-f]{2}){5}$)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString LeaderIdentity; 

	/**  Data structure for PTP synchronization priorities */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIO_PTPLeaderPriorities LeaderPriorities;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double LeaderAccuracy = 0.0;

	/** Enum string property [GNSS, Atomic clock, NTP] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString LeaderTimeSource = TEXT("NTP");

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double MeanPathDelay = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Vlan = 0;
};

/**
 * Object describing how the tracking device is synchronized for this sample.
 *  
 * frequency:
 *
 *  The frequency of a synchronization signal.This may differ from the sample frame rate for example in a genlocked tracking device. This is
 *  not required if the synchronization source is PTP or NTP.
 *
 * locked: Is the tracking device locked to the synchronization source
 *
 * offsets: Offsets in seconds between sync and sample. Critical for e.g.
 *   frame remapping, or when using different data sources for
 *   position/rotation and lens encoding
 *
 * present: Is the synchronization source present (a synchronization source can be present but not locked if frame rates differ)
 *
 * ptp: If the synchronization source is a PTP leader, then this object
 * 
 * contains:
 *
 *  - "profile": Specifies the PTP profile in use. This defines the operational rules and parameters for synchronization. For example "SMPTE
 *  ST2059-2:2021" for SMPTE 2110 based systems, or "IEEE Std 1588-2019" or "IEEE Std 802.1AS-2020" for industrial applications
 *
 *  - "domain": Identifies the PTP domain the device belongs to. Devices in the same domain can synchronize with each other
 *  
 *  - "leaderIdentity": The unique identifier (usually MAC address) of the current PTP leader
 *  
 *  - "leaderPriorities": The priority values of the leader used in the Best Clock Algorithm (BMCA). Lower values indicate higher priority
 *  
 *  - "priority1": Static priority set by the administrator
 *  
 *  - "priority2": Dynamic priority based on the leader’s role or clock quality
 *  
 *  - "leaderAccuracy": The timing offset in seconds from the sample timestamp to the PTP timestamp
 *  
 *  - "meanPathDelay": The average round-trip delay between the device and the PTP leader, measured in seconds
 *  
 * source: The source of synchronization must be defined as one of the
 * 
 * following:
 * 
 *  - "vlan": Integer representing the VLAN ID for PTP traffic (e.g., 100 for
 *  VLAN 100)
 *  
 *  - "leaderTimeSource": Indicates the leader’s source of time, such as GNSS, atomic clock, or NTP
 *    
 *  - "genlock": The tracking device has an external black/burst or tri-level analog sync signal that is triggering the capture of tracking samples
 *  
 *  - "videoIn": The tracking device has an external video signal that is triggering the capture of tracking samples
 *  
 *  - "ptp": The tracking device is locked to a PTP leader
 *  
 *  - "ntp": The tracking device is locked to an NTP server
 *  
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOSynchronization
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOSynchronization DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOSynchronization&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOSynchronization&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	bool Locked = false;

	/** Enum as string value can be "genlock", "videoIn", "ptp", ... */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Source;

	/** Synchonization rate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOFrameRate Frequency;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOSynchronizationOffsets Offsets;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	bool Present = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIO_PTP Ptp;
};

/**
 * SMPTE timecode of the sample. Timecode is a standard for labeling
 * individual frames of data in media systems and is useful for
 * inter-frame synchronization. Frame rate is a rational number, allowing
 * drop frame rates such as that colloquially called 29.97 to be
 * represented exactly, as 30000/1001. The timecode frame rate may differ
 * from the sample frequency. The zero-based sub-frame field allows for finer
 * division of the frame, e.g. interlaced frames have two sub-frames,
 * one per field.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOTimecode
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOTimecode DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOTimecode&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOTimecode&) const = default;
	
	FQualifiedFrameTime GetQualifiedFrameTime() const
	{
		const double SubFrameFraction = FMath::Clamp<double>(
			static_cast<double>(SubFrame) / FMath::Max(1, SubframesPerFrame), 
			0.0, 
			1.0
		);

		const bool bDropFrameFlag = FTimecode::UseDropFormatTimecode(FrameRate.GetFrameRate());

		return FQualifiedFrameTime(
			FTimecode(Hours, Minutes, Seconds, Frames, static_cast<float>(SubFrameFraction), bDropFrameFlag),
			FrameRate.GetFrameRate()
		);
	}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")	
	int32 Hours = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Minutes = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Seconds = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Frames = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOFrameRate FrameRate;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 SubFrame = 0;

	/** Number of subframes per frame. 1 is the minimum */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 SubframesPerFrame = 1;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOTiming
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOTiming DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOTiming&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOTiming&) const = default;
	
	/**
	 *  Enumerated value indicating whether the sample transport mechanism provides
	 *  inherent (’external’) timing, or whether the transport mechanism lacks inherent timing
	 *  and so the sample must contain a PTP timestamp itself (’internal’) to carry timing information.
	 **/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Mode;

	/** PTP timestamp of the data recording instant, provided for convenience during playback of e.g. pre-recorded tracking data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOTimestamp RecordedTimestamp;

	/**
	 * Sample frame rate as a rational number. Drop frame rates such as 29.97 should be
	 * represented as e.g. 30000/1001. In a variable rate system this should be estimated from the last
	 * sample delta time.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOFrameRate SampleRate;

	/** 
	 * PTP timestamp of the data recording instant, provided for convenience during playback of e.g. pre-recorded tracking data.
	 * The timestamp comprises a 48-bit unsigned integer (seconds), a 32-bit unsigned integer (nanoseconds)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOTimestamp SampleTimestamp;

	/** Integer incrementing with each sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 SequenceNumber = 0;

	/** Synchronization object for the timing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOSynchronization Synchronization;

	/** SMTPE timecode of the sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOTimecode Timecode;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens_DistortionOffset
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens_DistortionOffset DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens_DistortionOffset&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens_DistortionOffset&) const = default;

	FVector2D AsVector() const
	{
		return FVector2D(X,Y);
	}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float X = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float Y = 0;
};

/**
 * Encoders are represented in this way (as opposed to raw integer
 * values) to ensure values remain independent of encoder resolution,
 * minimum and maximum (at an acceptable loss of precision).
 * These values are only relevant in lenses with end-stops that
 * demarcate the 0 and 1 range.
 * 
 * Value should be provided in the following directions (if known):
 * Focus:   0=infinite     1=closest
 * Iris:    0=open         1=closed
 * Zoom:    0=wide angle   1=telephoto
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens_Encoders
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens_Encoders DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens_Encoders&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens_Encoders&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float Focus = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float Iris = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float Zoom = 0;
};


/**
 * Raw encoder values for focus, iris and zoom.
 * These values are dependent on encoder resolution and before any
 * homing / ranging has taken place.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens_RawEncoders
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens_RawEncoders DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens_RawEncoders&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens_RawEncoders&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Focus = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Iris = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int32 Zoom = 0;
};

/**
 * Coefficients for calculating the exposure fall-off (vignetting) of a lens.
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens_ExposureFalloff
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens_ExposureFalloff DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens_ExposureFalloff&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens_ExposureFalloff&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float A1 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float A2 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float A3 = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens_DistortionCoeff
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens_DistortionCoeff DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens_DistortionCoeff&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens_DistortionCoeff&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<float> Radial;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<float> Tangential;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float Overscan = 0.0;

	/** This is not currently in the spec but it would simplify sending any set of unnamed parameters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<float> Custom;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOLens
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOLens DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOLens&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOLens&) const = default;
	
	/**
	 * This list provides optional additional custom coefficients that can extend the existing lens model. The meaning
	 * of and how these characteristics are to be applied to a virtual camera would require
	 * negotiation between a particular producer and consumer.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<float> Custom;

	/**
	 * A list of Distortion objects that each define the coefficients for calculating the distortion characteristics of a lens comprising radial
	 * distortion coefficients of the spherical distortion (k1-N) and (optionally) the tangential distortion (p1-N). The key ’model’ names the
	 * distortion model. Typical values for ’model’ include "Brown-Conrady D-U" when mapping distorted to undistorted coordinates, and "Brown-Conrady
	 * U-D" when mapping undistorted to undistorted coordinates. If not provided, the default model is "Brown-Conrady D-U".
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<FLiveLinkOpenTrackIOLens_DistortionCoeff> Distortion;

	/**
	 *  Offset in x and y of the centre of distortion of the virtual camera in millimeter
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens_DistortionOffset DistortionOffset;

	/** Normalised real numbers (0-1) for focus, iris and zoom */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens_Encoders Encoders;

	/**
	 * Offset of the entrance pupil relative to the nominal imaging plane
	 * (positive if the entrance pupil is located on the side of the nominal
	 * imaging plane that is towards the object, and negative otherwise).
	 * Measured in meters as in a render engine it is often applied in the
	 * virtual camera’s transform chain.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	float EntrancePupilOffset = 0.0f;

	/**
	 * Coefficients for calculating the exposure fall-off (vignetting) of a lens
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens_ExposureFalloff ExposureFalloff;

	/** The linear f-number of the lens, equal to the focal length divided by the diameter of the entrance pupil. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat FStop;

	/** Distance between the pinhole and the image plane in the simple CGI pinhole camera model. (millimeters) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat PinholeFocalLength;

	/** Focus distance/position of the lens in meters. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat FocusDistance;

	/**  Offset in x and y of the centre of perspective projection of the virtual camera */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens_DistortionOffset ProjectionOffset;

	/** Raw encoder values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens_RawEncoders RawEncoders;

	/** T-stop: Linear t-number of the lens, equal to the F-number of the lens divided by the square root of the transmittance of the lens. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FOpenTrackIOOptionalFloat TStop;
};

/**
 * Name of the protocol in which the sample is being employed, and
 * version of that protocol
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOProtocol
{
	GENERATED_BODY()

	FLiveLinkOpenTrackIOProtocol()
	{
		Version = TArray<int8>{1,0,0};
	};
	
	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOProtocol DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOProtocol&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOProtocol&) const = default;
	
	bool IsSupported() const
	{
		/** Validate that we support this version of OpenTrackIO. We assume that patch release are compatible. */
		return Name == TEXT("OpenTrackIO")
			&& Version.Num() >= 2
			&& Version[0] == 1
			&& Version[1] == 0;
	}

	/** The name of the protocol. If you want to extract the full version string then use GetVersionString().*/
	UPROPERTY(BlueprintReadOnly, Category = "OpenTrackIO")
	FString Name;

	/** Three numbers that represent the protocol version.  This is not accessible by Blueprints because int8 is not supported.  */
	UPROPERTY()
	TArray<int8> Version;
};

UCLASS(meta=(BlueprintThreadSafe, ScriptName = "OpenTrackIO"), MinimalAPI)
class ULiveLinkOpenTrackIOLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/** A string representing the protocol name plus version. */
	UFUNCTION(BlueprintPure, Category = "OpenTrackIO")
	static FString GetVersionString(const FLiveLinkOpenTrackIOProtocol& Protocol)
	{
		int Major = Protocol.Version.Num() > 0 ? Protocol.Version[0] : 1;
		int Minor = Protocol.Version.Num() > 1 ? Protocol.Version[1] : 0;
		int Patch = Protocol.Version.Num() > 2 ? Protocol.Version[2] : 0;
		return FString::Printf(TEXT("%s %d.%d.%d"),	*Protocol.Name, Major, Minor, Patch);
	}
};


/**
 * Position of stage origin in global ENU and geodetic coordinates
 * (E, N, U, lat0, lon0, h0). Note this may be dynamic if the stage is
 * inside a moving vehicle. Units represented in meters. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOGlobalStage
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOGlobalStage DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOGlobalStage&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOGlobalStage&) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double E = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double N = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double U = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Lat0 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Lon0 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double H0 = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIO_Rotator
{
	GENERATED_BODY()
	
	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIO_Rotator DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIO_Rotator&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIO_Rotator&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Pan = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Tilt = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Roll = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIO_XYZ
{
	GENERATED_BODY()

	FLiveLinkOpenTrackIO_XYZ() = default;
	FLiveLinkOpenTrackIO_XYZ(double InX, double InY, double InZ)
		: X(InX), Y(InY), Z(InZ)
	{
	}

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIO_XYZ DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIO_XYZ&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIO_XYZ&) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double X = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Y = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	double Z = 0;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOTransform
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOTransform DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOTransform&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOTransform&) const = default;
	
	/** Translation part of a OpenTrackIO xform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIO_XYZ Translation;
	
	/** Rotation part of a OpenTrackIO xform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIO_Rotator Rotation;

	/** Scale part of a OpenTrackIO xform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIO_XYZ Scale = FLiveLinkOpenTrackIO_XYZ(1,1,1);

	/** OpenTrackIO identifier (can be empty) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Id;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOCustomDataField
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOCustomDataField DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOCustomDataField&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOCustomDataField&) const = default;

	/** Meta data is inserted into Live Link Frame data using a Key, Value pair. This is the key part of that pair. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Key;

	/** Meta data is inserted into Live Link Frame data using a Key, Value pair. This is the value part of that pair. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString Value;
};

USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOCustomData
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOCustomData DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOCustomData&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOCustomData&) const = default;

	/** A list of string based key/value pairs that can be read and applied to the Live Link data stream. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<FLiveLinkOpenTrackIOCustomDataField> LiveLinkMetaData; 
};

/**
 * Data from a parsed header that was read by other CBOR or JSON format.
 *
 * Schema can be found here: https://www.opentrackio.org/schema.json
 * 
 */
USTRUCT(BlueprintType)
struct FLiveLinkOpenTrackIOData
{
	GENERATED_BODY()

	/** Returns true if the struct is equivalent to the default struct values. */
	bool IsDefault() const
	{
		static FLiveLinkOpenTrackIOData DefaultValue;
		return DefaultValue == *this;
	}
	
	bool operator==(const FLiveLinkOpenTrackIOData&) const = default;
	bool operator!=(const FLiveLinkOpenTrackIOData&) const = default;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOStatics Static;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOTracker Tracker;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOTiming Timing;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOLens Lens;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOProtocol Protocol;

	/**
	 * URN serving as unique identifier of the sample in which data is being transported.
	 * Pattern -> ^urn:uuid:[0-9a-f]{8}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{12}$
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString SampleId;

	/**
	 * URN serving as unique identifier of the source from which data is being transported.
	 * Pattern -> ^urn:uuid:[0-9a-f]{8}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{4}-[0-9a-f]{12}$
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FString SourceId;

	/**
	 * Number that identifies the index of the stream from a source from which
	 * data is being transported. This is most important in the case where a source
	 * is producing multiple streams of samples.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	int64 SourceNumber = 0;

	/**
	 * List of sampleId properties of samples related to this sample. The
	 * existence of a sample with a given sampleId is not guaranteed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<FString> RelatedSampleIds;

	/** Position of stage origin in global ENU and geodetic coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOGlobalStage GlobalStage;

	 /**
	  * A list of transforms. Transforms are composed in order with the last in the list representing the X,Y,Z in meters of camera sensor relative to
	  * stage origin. The Z axis points upwards and the coordinate system is right-handed. Y points in the forward camera direction (when pan, tilt
	  * and roll are zero). For example in an LED volume Y would point towards the centre of the LED wall and so X would point to camera-right.
	  * Rotation expressed as euler angles in degrees of the camera sensor relative to stage origin Rotations are intrinsic and are measured around
	  * the axes ZXY, commonly referred to as [pan, tilt, roll] Notes on Euler angles: Euler angles are human readable and unlike quarternions,
	  * provide the ability for cycles (with angles >360 or <0 degrees). Where a tracking system is providing the pose of a virtual camera, gimbal
	  * lock does not present the physical challenges of a robotic system. Conversion to and from quarternions is trivial with an acceptable loss of
	  * precision.
	  */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	TArray<FLiveLinkOpenTrackIOTransform> Transforms;

	/**
	 * Typed property to allow users to add custom data to the incoming OpenTrackIO data. Our implementation only supports a field called LiveLinkMetaData.
	 * For example,
	 * 
	 * 	"custom" : {
	 *     "liveLinkMetaData" : [
	 *			{
	 *				"key": "mykey",
	 *				"value": "myvalue"
	 *			},
	 *			{
	 *				"key": "otherKey",
	 *				"value": "otherValue"
	 *			}
	 *		]
	 *  }
	 * */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenTrackIO")
	FLiveLinkOpenTrackIOCustomData Custom;
};


namespace UE::OpenTrackIO
{
	/** Brown-Conrady D-U, the default for nameless distortion model per opentrackio.org */
	static const FName BrownConradyDU = TEXT("Brown-Conrady D-U");
	
	/** Brown-Conrady U-D distortion model */
	static const FName BrownConradyUD = TEXT("Brown-Conrady U-D");

	/** Name of the FOpenTrackIOOptional "bIsSet" property */
	static const FName OptionalTypeIsSetName = GET_MEMBER_NAME_CHECKED(FOpenTrackIOOptionalFloat, bIsSet);

	/** Name of the FOpenTrackIOOptional "Value" property */
	static const FName OptionalTypeValueName = GET_MEMBER_NAME_CHECKED(FOpenTrackIOOptionalFloat, Value);

	/**
	 * Returns true if Struct is one of our custom optional USTRUCT types.
	 */
	static bool IsOpenTrackIOOptionalType(const UScriptStruct* ScriptStruct)
	{
		static UScriptStruct* OptionalStructs[] = {
			FOpenTrackIOOptionalFloat::StaticStruct(),
			FOpenTrackIOOptionalInt32::StaticStruct(),
			FOpenTrackIOOptionalDouble::StaticStruct()
		};

		for (UScriptStruct* OptionalStruct : OptionalStructs)
		{
			if (ScriptStruct == OptionalStruct)
			{
				return true;
			}
		}

		return false;
	}
}
