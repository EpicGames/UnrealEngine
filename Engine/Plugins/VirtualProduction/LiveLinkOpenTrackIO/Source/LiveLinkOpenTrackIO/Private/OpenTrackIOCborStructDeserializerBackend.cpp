// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenTrackIOCborStructDeserializerBackend.h"

#include "LiveLinkOpenTrackIO.h"
#include "LiveLinkOpenTrackIOLiveLinkTypes.h"

#include "UObject/Class.h"
#include "UObject/AnsiStrProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyOptional.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/Utf8StrProperty.h"


namespace UE::OpenTrackIO::Cbor::Private
{
	/**
	* Clears the value of the given property.
	*
	* @param Property The property to clear.
	* @param Outer The property that contains the property to be cleared, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to clear (if the property is an array).
	* 
	* @return true on success, false otherwise.
	*/
	static bool ClearPropertyValue(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
	{
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer);

		if (ArrayProperty != nullptr)
		{
			if (ArrayProperty->Inner != Property)
			{
				return false;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			ArrayIndex = ArrayHelper.AddValue();
		}

		Property->ClearValue_InContainer(Data, ArrayIndex);

		return true;
	}


	/**
	* Gets a pointer to object of the given property.
	*
	* @param Property The property to get.
	* @param Outer The property that contains the property to be get, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to set (if the property is an array).
	* 
	* @return A pointer to the object represented by the property, null otherwise.
	*/
	static void* GetPropertyValuePtr(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
	{
		check(Property);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer))
		{
			if (ArrayProperty->Inner != Property)
			{
				return nullptr;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			int32 Index = ArrayHelper.AddValue();

			return ArrayHelper.GetRawPtr(Index);
		}

		if (ArrayIndex >= Property->ArrayDim)
		{
			return nullptr;
		}

		return Property->template ContainerPtrToValuePtr<void>(Data, ArrayIndex);
	}

	/**
	* Sets the value of the given property.
	*
	* @param Property The property to set.
	* @param Outer The property that contains the property to be set, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to set (if the property is an array).
	* 
	* @return true on success, false otherwise.
	*/
	template<typename PropertyType, typename ValueType>
	static bool SetPropertyValue(PropertyType* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const ValueType& Value)
	{
		if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
		{
			Property->SetPropertyValue(Ptr, Value);
			return true;
		}

		return false;
	}

	/**
	 * Sets a numeric UProperty (int, uint, float, or double) on a given object or struct.
	 *
	 * @tparam T           Numeric type to write (e.g. float, double, int8, uint64, etc.).
	 * @param Property     Target property.
	 * @param Outer        The owning property of the target property.
	 * @param Data         Pointer to the raw memory container holding the property value (object or struct).
	 * @param ArrayIndex   Index within a TArray property. Use 0 for non-array properties.
	 * @param InValue      The value to assign. Will be cast to the property's actual type.
	 * 
	 * @return             True if Property was a numeric type and the value was written. false otherwise.
	 */
	template<typename T>
	static bool SetNumericPropertyValue(
		FProperty* Property,
		FProperty* Outer,
		void* Data,
		int32 ArrayIndex,
		T InValue)
	{
		if (FNumericProperty* NumProp = CastField<FNumericProperty>(Property))
		{
			void* ValuePtr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex);

			if (!ValuePtr)
			{
				return false;
			}

			if (NumProp->IsFloatingPoint())
			{
				NumProp->SetFloatingPointPropertyValue(ValuePtr, static_cast<double>(InValue));
			}
			else // integer or enum
			{
				if constexpr (std::is_signed_v<T>)
				{
					NumProp->SetIntPropertyValue(ValuePtr, static_cast<int64>(InValue));
				}
				else
				{
					NumProp->SetIntPropertyValue(ValuePtr, static_cast<uint64>(InValue));
				}
			}
			return true;
		}

		return false;
	}

	/**
	 * Attempts to read one of our optional USTRUCTs. Returns true on successful parse,
	 * false otherwise (leaving bIsSet unchanged on failure).
	 */
	bool TryReadOurOptionalStruct(
		FOpenTrackIOCborStructDeserializerBackend* Backend,
		const UScriptStruct* OptionalStruct,
		FStructProperty* StructProp,
		FProperty* Outer,
		void* Data,
		int32 ArrayIndex)
	{
		void* StructData = GetPropertyValuePtr(StructProp, Outer, Data, ArrayIndex);
		if (!StructData)
		{
			return false;
		}

		if (FProperty* ValueProp = OptionalStruct->FindPropertyByName(OptionalTypeValueName))
		{
			if (!Backend->ReadProperty(ValueProp, StructProp, StructData, /* Inner ArrayIndex */ 0))
			{
				return false;
			}

			if (FBoolProperty* BoolProp = CastField<FBoolProperty>(OptionalStruct->FindPropertyByName(OptionalTypeIsSetName)))
			{
				BoolProp->SetPropertyValue_InContainer(StructData, true);
			}

			return true;
		}
		return false;
	}
};


FOpenTrackIOCborStructDeserializerBackend::FOpenTrackIOCborStructDeserializerBackend(FArchive& Archive)
	: CborReader(&Archive, ECborEndianness::StandardCompliant)
{}

FOpenTrackIOCborStructDeserializerBackend::~FOpenTrackIOCborStructDeserializerBackend() = default;

const FString& FOpenTrackIOCborStructDeserializerBackend::GetCurrentPropertyName() const
{
	return LastMapKey;
}

FString FOpenTrackIOCborStructDeserializerBackend::GetDebugString() const
{
	FArchive* Ar = const_cast<FArchive*>(CborReader.GetArchive());
	return FString::Printf(TEXT("Offset: %" UINT64_FMT), Ar ? Ar->Tell() : 0);
}

const FString& FOpenTrackIOCborStructDeserializerBackend::GetLastErrorMessage() const
{
	static FString Dummy;
	return Dummy;
}

namespace UE::OpenTrackIOCborStructDeserializerBackend::Private
{

FString ConvertCborTypeToString(ECborCode CborCode)
{
	switch (CborCode)
	{
		case ECborCode::Uint:
			return TEXT("Uint");
		case ECborCode::Int:
			return TEXT("Int");
		case ECborCode::ByteString:
			return TEXT("ByteString");
		case ECborCode::TextString:
			return TEXT("TextString");						
		case ECborCode::Array:
			return TEXT("Array");
		case ECborCode::Map:
			return TEXT("Map");
		case ECborCode::Tag:
			return TEXT("Tag");
		case ECborCode::Prim:
			return TEXT("Prim");
		// Fallthrough
	};
	return TEXT("InvalidType");
}

} // namespace UE::CborStructDeserializerBackend::Private

bool FOpenTrackIOCborStructDeserializerBackend::GetNextToken(EStructDeserializerBackendTokens& OutToken)
{
	LastMapKey.Reset();

	if (bDeserializingByteArray) // Deserializing the content of a TArray<uint8>/TArray<int8> property?
	{
		if (DeserializingByteArrayIndex < LastContext.AsByteArray().Num())
		{
			OutToken = EStructDeserializerBackendTokens::Property; // Need to consume a byte from the CBOR ByteString as a UByteProperty/UInt8Property.
		}
		else
		{
			bDeserializingByteArray = false;
			OutToken = EStructDeserializerBackendTokens::ArrayEnd; // All bytes from the byte string were deserialized into the TArray<uint8>/TArray<int8>.
		}

		return true;
	}

	if (!CborReader.ReadNext(LastContext))
	{
		OutToken = LastContext.IsError() ? EStructDeserializerBackendTokens::Error : EStructDeserializerBackendTokens::None;
		return false;
	}

	if (LastContext.IsBreak())
	{
		ECborCode ContainerEndType = LastContext.AsBreak();
		// We do not support indefinite string container type
		check(ContainerEndType == ECborCode::Array || ContainerEndType == ECborCode::Map);
		OutToken = ContainerEndType == ECborCode::Array ? EStructDeserializerBackendTokens::ArrayEnd : EStructDeserializerBackendTokens::StructureEnd;
		return true;
	}

	// We've read an odd number of items from this map so far. So the next value is the key and it must be a string.
	if (CborReader.GetContext().MajorType() == ECborCode::Map && (CborReader.GetContext().AsLength() & 1))
	{
		// This code is designed to deserialize CBOR data into Unreal Engine structs. If this is not a string then we are somehow not dealing with UE
		// structs and this would be an error. Previously this code would just assert and crash.  This gracefully handles this case and passes the
		// failure down to the caller. 
		if (LastContext.MajorType() == ECborCode::TextString)
		{
			LastMapKey = LastContext.AsString();

			// Read next and carry on
			if (!CborReader.ReadNext(LastContext))
			{
				OutToken = LastContext.IsError() ? EStructDeserializerBackendTokens::Error : EStructDeserializerBackendTokens::None;
				return false;
			}
		}
		else
		{
			OutToken = EStructDeserializerBackendTokens::Error;
			FString CborMajorTypeAsString = UE::OpenTrackIOCborStructDeserializerBackend::Private::ConvertCborTypeToString(LastContext.MajorType());
			uint8 CborMajorTypeAsInt = static_cast<uint8>(LastContext.MajorType());
			UE_LOGF(LogSerialization, Error, "Encountered invalided context while parsing CBOR map. Expected a string type but got code %d (%ls)", CborMajorTypeAsInt, *CborMajorTypeAsString);
			return false;
		}
	}

	switch (LastContext.MajorType())
	{
	case ECborCode::Array:
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
		break;

	case ECborCode::Map:
		OutToken = EStructDeserializerBackendTokens::StructureStart;
		break;

	case ECborCode::ByteString: // Used for size optimization on TArray<uint8>/TArray<int8>. Might be replaced if https://datatracker.ietf.org/doc/draft-ietf-cbor-array-tags/ is adopted.
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
		DeserializingByteArrayIndex = 0;
		bDeserializingByteArray = true;
		break;

	case ECborCode::Int:
		// fall through
	case ECborCode::Uint:
		// fall through
	case ECborCode::TextString:
		// fall through
	case ECborCode::Prim:
		OutToken = EStructDeserializerBackendTokens::Property;
		break;

	default:
		// Other types are unsupported
		return false;
	}

	return true;
}

bool FOpenTrackIOCborStructDeserializerBackend::ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	// Support for our custom USTRUCT replacement of TOptionals, since TOptionals are not currently supported in blueprint,
	// and we need that to effectively expose many of these optional parameters.
	//
	if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
	{
		const UScriptStruct* UDStruct = StructProp->Struct;

		if (UE::OpenTrackIO::IsOpenTrackIOOptionalType(UDStruct))
		{
			return UE::OpenTrackIO::Cbor::Private::TryReadOurOptionalStruct(this, UDStruct, StructProp, Outer, Data, ArrayIndex);
		}
	}

	// Unwrap TOptional and recurse back into this function
	if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
	{
		void* ValuePtr = UE::OpenTrackIO::Cbor::Private::GetPropertyValuePtr(OptionalProperty, Outer, Data, ArrayIndex);

		if (!ValuePtr)
		{
			return false;
		}

		// "Set" the optional and get a pointer to its value

		void* InnerData = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(ValuePtr);

		if (!InnerData)
		{
			return false;
		}

		// Get the property inside the optional

		FProperty* ValueProp = OptionalProperty->GetValueProperty();

		if (!ValueProp)
		{
			return false;
		}

		// Recurse back into this function to handle the inner property
		const bool bInnerPropertyWasSet = ReadProperty(ValueProp, Property, InnerData, ArrayIndex);

		// But if the inner property wasn't read into, then reset the optional
		if (!bInnerPropertyWasSet)
		{
			OptionalProperty->MarkUnset(ValuePtr);
		}

		return bInnerPropertyWasSet;
	}

	switch (LastContext.MajorType())
	{
	// Unsigned Integers
	case ECborCode::Uint:
	{
		const bool bWasSet = UE::OpenTrackIO::Cbor::Private::SetNumericPropertyValue(Property, Outer, Data, ArrayIndex, LastContext.AsUInt());

		if (!bWasSet)
		{
			UE_LOG(LogLiveLinkOpenTrackIO, Verbose, TEXT("Unsigned integer field %s with value '%" UINT64_FMT "' is not supported in FProperty type %s (%s)"),
				*Property->GetFName().ToString(), LastContext.AsUInt(), *Property->GetClass()->GetName(), *GetDebugString());
		}

		return bWasSet;
	}

	// Signed Integers
	case ECborCode::Int:
	{
		const bool bWasSet = UE::OpenTrackIO::Cbor::Private::SetNumericPropertyValue(Property, Outer, Data, ArrayIndex, LastContext.AsInt());

		if (!bWasSet)
		{
			UE_LOG(LogLiveLinkOpenTrackIO, Verbose, TEXT("Integer field %s with value '%" UINT64_FMT "' is not supported in FProperty type %s (%s)"),
				*Property->GetFName().ToString(), LastContext.AsUInt(), *Property->GetClass()->GetName(), *GetDebugString());
		}

		return bWasSet;
	}

	// Strings, Names, Enumerations & Object/Class reference
	case ECborCode::TextString:
	{
		if (FAnsiStrProperty* AnsiStrProperty = CastField<FAnsiStrProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(AnsiStrProperty, Outer, Data, ArrayIndex, LastContext.AsAnsiString());
		}

		if (FUtf8StrProperty* Utf8StrProperty = CastField<FUtf8StrProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(Utf8StrProperty, Outer, Data, ArrayIndex, LastContext.AsUtf8String());
		}

		const FString StringValue = LastContext.AsString();

		if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(StrProperty, Outer, Data, ArrayIndex, StringValue);
		}

		if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(NameProperty, Outer, Data, ArrayIndex, FName(*StringValue));
		}

		if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
		{
			FText TextValue;
			if (!FTextStringHelper::ReadFromBuffer(*StringValue, TextValue))
			{
				TextValue = FText::FromString(StringValue);
			}
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(TextProperty, Outer, Data, ArrayIndex, TextValue);
		}

		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (!ByteProperty->Enum)
			{
				return false;
			}

			int64 Value = ByteProperty->Enum->GetValueByName(*StringValue);
			if (Value == INDEX_NONE)
			{
				return false;
			}

			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)Value);
		}

		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			int64 Value = EnumProperty->GetEnum()->GetValueByName(*StringValue);
			if (Value == INDEX_NONE)
			{
				return false;
			}

			if (void* ElementPtr = UE::OpenTrackIO::Cbor::Private::GetPropertyValuePtr(EnumProperty, Outer, Data, ArrayIndex))
			{
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ElementPtr, Value);
				return true;
			}

			return false;
		}

		if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(ClassProperty, Outer, Data, ArrayIndex, LoadObject<UClass>(NULL, *StringValue, NULL, LOAD_NoWarn));
		}

		if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(SoftClassProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
		}

		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(ObjectProperty, Outer, Data, ArrayIndex, StaticFindObject(ObjectProperty->PropertyClass, nullptr, *StringValue));
		}

		if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(WeakObjectProperty, Outer, Data, ArrayIndex, FWeakObjectPtr(StaticFindObject(WeakObjectProperty->PropertyClass, nullptr, *StringValue)));
		}

		if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(SoftObjectProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
		}

		UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "String field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), *StringValue, *Property->GetClass()->GetName(), *GetDebugString());

		return false;
	}

	// Stream of bytes: Used for TArray<uint8>/TArray<int8>
	case ECborCode::ByteString:
	{
		check(bDeserializingByteArray);

		// Consume one byte from the byte string.
		TArrayView<const uint8> DeserializedByteArray = LastContext.AsByteArray();
		check(DeserializingByteArrayIndex < DeserializedByteArray.Num());
		uint8 ByteValue = DeserializedByteArray[DeserializingByteArrayIndex++];

		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, ByteValue);
		}
		else if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
		{
			return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(Int8Property, Outer, Data, ArrayIndex, (int8)ByteValue);
		}

		UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "Error while deserializing field %ls. Unexpected UProperty type %ls. Expected a UByteProperty/UInt8Property to deserialize a TArray<uint8>/TArray<int8>", *Property->GetFName().ToString(), *Property->GetClass()->GetName());
		return false;
	}

	// Prim
	case ECborCode::Prim:
	{
		switch (LastContext.AdditionalValue())
		{
		// Boolean
		case ECborCode::True:
			// fall through
		case ECborCode::False:
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				return UE::OpenTrackIO::Cbor::Private::SetPropertyValue(BoolProperty, Outer, Data, ArrayIndex, LastContext.AsBool());
			}
			UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "Boolean field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsBool() ? *(CoreTexts.True.ToString()) : *(CoreTexts.False.ToString()), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
		}

		// Null
		case ECborCode::Null:
		{
			return UE::OpenTrackIO::Cbor::Private::ClearPropertyValue(Property, Outer, Data, ArrayIndex);
		}

		// Float
		case ECborCode::Value_4Bytes:
		{
			const bool bWasSet = UE::OpenTrackIO::Cbor::Private::SetNumericPropertyValue(Property, Outer, Data, ArrayIndex, LastContext.AsFloat());

			if (!bWasSet)
			{
				UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "Float field %ls with value '%f' is not supported in FProperty type %ls (%ls)",
					*Property->GetFName().ToString(), LastContext.AsFloat(), *Property->GetClass()->GetName(), *GetDebugString());
			}

			return bWasSet;
		}

		// Double
		case ECborCode::Value_8Bytes:
		{
			const bool bWasSet = UE::OpenTrackIO::Cbor::Private::SetNumericPropertyValue(Property, Outer, Data, ArrayIndex, LastContext.AsDouble());

			if (!bWasSet)
			{
				UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "Double field %ls with value '%f' is not supported in FProperty type %ls (%ls)",
					*Property->GetFName().ToString(), LastContext.AsFloat(), *Property->GetClass()->GetName(), *GetDebugString());
			}

			return bWasSet;
		}

		default:
			UE_LOGF(LogLiveLinkOpenTrackIO, Verbose, "Unsupported primitive type for %ls with value '%f' in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsDouble(), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
		}
	}
	}

	return true;
}

bool FOpenTrackIOCborStructDeserializerBackend::ReadPODArray(FArrayProperty* ArrayProperty, void* Data)
{
	// if we just read a byte array, copy the full array if the inner property is of the appropriate type 
	if (bDeserializingByteArray
		&& (CastField<FByteProperty>(ArrayProperty->Inner) || CastField<FInt8Property>(ArrayProperty->Inner)))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
		TArrayView<const uint8> DeserializedByteArray = LastContext.AsByteArray();
		if (DeserializedByteArray.Num())
		{
			ArrayHelper.AddUninitializedValues(DeserializedByteArray.Num());
			void* ArrayStart = ArrayHelper.GetRawPtr();
			FMemory::Memcpy(ArrayStart, DeserializedByteArray.GetData(), DeserializedByteArray.Num());
		}
		bDeserializingByteArray = false;
		return true;
	}
	return false;
}

void FOpenTrackIOCborStructDeserializerBackend::SkipArray()
{
	if (bDeserializingByteArray) // Deserializing a TArray<uint8>/TArray<int8> property as byte string?
	{
		check(DeserializingByteArrayIndex == 0);
		bDeserializingByteArray = false;
	}
	else
	{
		CborReader.SkipContainer(ECborCode::Array);
	}
}

void FOpenTrackIOCborStructDeserializerBackend::SkipStructure()
{
	CborReader.SkipContainer(ECborCode::Map);
}
