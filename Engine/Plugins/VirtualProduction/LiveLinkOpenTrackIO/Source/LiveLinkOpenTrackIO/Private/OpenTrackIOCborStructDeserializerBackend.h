// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CborReader.h"
#include "CborTypes.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IStructDeserializerBackend.h"

class FArchive;
class FArrayProperty;
class FProperty;

/**
 * Implements a reader for UStruct deserialization using Cbor.
 */
class FOpenTrackIOCborStructDeserializerBackend
	: public IStructDeserializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 * 
	 * @param Archive The archive to deserialize from.
	 */
	FOpenTrackIOCborStructDeserializerBackend(FArchive& Archive);
	virtual ~FOpenTrackIOCborStructDeserializerBackend();

public:

	// IStructDeserializerBackend interface
	virtual const FString& GetCurrentPropertyName() const override;
	virtual FString GetDebugString() const override;
	virtual const FString& GetLastErrorMessage() const override;
	virtual bool GetNextToken(EStructDeserializerBackendTokens& OutToken) override;
	virtual bool ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) override;
	virtual bool ReadPODArray(FArrayProperty* ArrayProperty, void* Data) override;
	virtual void SkipArray() override;
	virtual void SkipStructure() override;

private:
	/** Holds the Cbor reader used for the actual reading of the archive. */
	FCborReader CborReader;

	/** Holds the last read Cbor Context. */
	FCborContext LastContext;

	/** Holds the last map key. */
	FString LastMapKey;

	/** The index of the next byte to copy from the CBOR byte stream into the corresponding TArray<uint8>/TArray<int8> property. */
	int32 DeserializingByteArrayIndex = 0;

	/** Whether a TArray<uint8>/TArray<int8> property is being deserialized. */
	bool bDeserializingByteArray = false;
};
