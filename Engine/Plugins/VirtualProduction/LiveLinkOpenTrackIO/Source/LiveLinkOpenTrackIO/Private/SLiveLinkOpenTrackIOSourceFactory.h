// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkOpenTrackIOConnectionSettings.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#if WITH_EDITOR
#include "IStructureDetailsView.h"
#endif //WITH_EDITOR

#include "Input/Reply.h"

struct FLiveLinkOpenTrackIOConnectionSettings;

DECLARE_DELEGATE_OneParam(FOnLiveLinkOpenTrackIOConnectionSettingsAccepted, FLiveLinkOpenTrackIOConnectionSettings);

class SLiveLinkOpenTrackIOSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkOpenTrackIOSourceFactory)
	{}
		SLATE_EVENT(FOnLiveLinkOpenTrackIOConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);


private:
	FLiveLinkOpenTrackIOConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif //WITH_EDITOR

	FReply OnSettingsAccepted();
	FOnLiveLinkOpenTrackIOConnectionSettingsAccepted OnConnectionSettingsAccepted;
};