// Copyright Epic Games, Inc. All Rights Reserved.
// 
#include "OpenTrackIOTestHelpers.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

namespace UE::OpenTrackIO::Tests
{

FString GetSampleFile(const FString& InName)
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	FString FilePath = FPaths::Combine(Plugin->GetBaseDir(),
		TEXT("Source"), TEXT("LiveLinkOpenTrackIO"),
		TEXT("Private"), TEXT("Tests"), TEXT("Samples"), InName);
	ensure(FPaths::FileExists(FilePath));
	return FilePath;
}	

}

