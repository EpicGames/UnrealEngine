// Copyright Epic Games, Inc. All Rights Reserved.

// clang-format off
#include "OpenTrackIOTestHelpers.h"

#if WITH_EDITOR

#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"

#include "LiveLinkOpenTrackIOParser.h"
#include "LiveLinkOpenTrackIOTypes.h"

#if WITH_DEV_AUTOMATION_TESTS

// clang-format on
namespace UE::OpenTrackIO::Tests
{

BEGIN_DEFINE_SPEC(FOpenTrackIOReaderTests,
	TEXT("Editor.LiveLinkOpenTrackIO.ReaderTests"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
const TArray<FString> Tests = {
	TEXT("FullStaticOpenTrack"),
	TEXT("RecommendedDynamicExample"),
	TEXT("CompleteStaticExample")
};

END_DEFINE_SPEC(FOpenTrackIOReaderTests)

void FOpenTrackIOReaderTests::Define()
{
	using namespace UE::OpenTrackIO::Tests;

	BeforeEach([this] {
		// Nothing right now.
	});
	Describe("CanonicalCases", [this] {
		It("Parses JSON", [this] {
			for (const FString& TestName : Tests)
			{
				FString JsonTestName = TestName + TEXT(".json");
				FString FullPath = GetSampleFile(TestName + TEXT(".json"));

				FString JsonBlob;
				if (TestTrue("Parsed JSON -> " + JsonTestName, FFileHelper::LoadFileToString(JsonBlob, *FullPath)))
				{
					TOptional<FLiveLinkOpenTrackIOData> Data = UE::OpenTrackIO::Private::ParseJsonBlob(JsonBlob);
					TestTrue(JsonTestName + " JSON should be successful.", Data.IsSet());
				}
			}
		});

		It("Fails with invalid JSON", [this] {
			FString InvalidJson = TEXT("{\"not opentrack\" : {}}");
			TOptional<FLiveLinkOpenTrackIOData> Data = UE::OpenTrackIO::Private::ParseJsonBlob(InvalidJson);

			TestTrue("Should have failed with bogus data", !Data.IsSet());
		});

		It("Parses CBOR", [this] {
			for (const FString& TestName : Tests)
			{
				FString CborTestName = TestName + TEXT(".cbor");
				FString FullPath = GetSampleFile(TestName + TEXT(".cbor"));

				TArray<uint8> BinaryBlob;
				if (TestTrue("Parsed CBOR -> " + CborTestName, FFileHelper::LoadFileToArray(BinaryBlob, *FullPath)))
				{
					TOptional<FLiveLinkOpenTrackIOData> Data = UE::OpenTrackIO::Private::ParseCborBlob(BinaryBlob);
					TestTrue(CborTestName + " CBOR should be successful.", Data.IsSet());
				}
			}
		});
	});
}

} // namespace UE::OpenTrackIO::Tests

	#endif // WITH_DEV_AUTOMATION_TESTS

#endif // WITH_EDITOR
