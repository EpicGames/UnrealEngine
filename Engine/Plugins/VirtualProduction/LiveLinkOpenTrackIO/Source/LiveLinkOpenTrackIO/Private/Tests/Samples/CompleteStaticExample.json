{
  "static": {
    "duration": {
      "num": 1,
      "denom": 25
    },
    "camera": {
      "captureFrameRate": {
        "num": 24000,
        "denom": 1001
      },
      "activeSensorPhysicalDimensions": {
        "height": 24.0,
        "width": 36.0
      },
      "activeSensorResolution": {
        "height": 2160,
        "width": 3840
      },
      "make": "CameraMaker",
      "model": "Model20",
      "serialNumber": "1234567890A",
      "firmwareVersion": "1.2.3",
      "label": "A",
      "anamorphicSqueeze": {
        "num": 1,
        "denom": 1
      },
      "isoSpeed": 4000,
      "fdlLink": "urn:uuid:e386a0b9-4f20-4767-88bd-8b6066a78123",
      "shutterAngle": 45.0
    },
    "lens": {
      "distortionOverscanMax": 1.2,
      "undistortionOverscanMax": 1.3,
      "make": "LensMaker",
      "model": "Model15",
      "serialNumber": "1234567890A",
      "nominalFocalLength": 14.0,
      "calibrationHistory": [
        "LensMaker 123",
        "TrackerMaker 123"
      ]
    },
    "tracker": {
      "make": "TrackerMaker",
      "model": "Tracker",
      "serialNumber": "1234567890A",
      "firmwareVersion": "1.2.3"
    }
  },
  "tracker": {
    "notes": "Example generated sample.",
    "recording": false,
    "slate": "A101_A_4",
    "status": "Optical Good"
  },
  "timing": {
    "mode": "internal",
    "recordedTimestamp": {
      "seconds": 1718806000,
      "nanoseconds": 500000000
    },
    "sampleRate": {
      "num": 24000,
      "denom": 1001
    },
    "sampleTimestamp": {
      "seconds": 1718806554,
      "nanoseconds": 500000000
    },
    "sequenceNumber": 0,
    "synchronization": {
      "locked": true,
      "source": "ptp",
      "frequency": {
        "num": 24000,
        "denom": 1001
      },
      "present": true,
      "ptp": {
        "profile": "SMPTE ST2059-2:2021",
        "domain": 1,
        "leaderIdentity": "00:11:22:33:44:55",
        "leaderPriorities": {
          "priority1": 128,
          "priority2": 128
        },
        "leaderAccuracy": 5e-08,
        "leaderTimeSource": "GNSS",
        "meanPathDelay": 0.000123,
        "vlan": 100
      }
    },
    "timecode": {
      "hours": 1,
      "minutes": 2,
      "seconds": 3,
      "frames": 4,
      "frameRate": {
        "num": 24000,
        "denom": 1001
      }
    }
  },
  "lens": {
    "custom": [
      1.0,
      2.0
    ],
    "distortion": [
      {
        "model": "Brown-Conrady U-D",
        "radial": [
          1.0,
          2.0,
          3.0,
          4.0,
          5.0,
          6.0
        ],
        "tangential": [
          1.0,
          2.0
        ],
        "overscan": 3.0
      },
      {
        "radial": [
          1.0,
          2.0,
          3.0,
          4.0,
          5.0,
          6.0
        ],
        "tangential": [
          1.0,
          2.0
        ],
        "overscan": 2.0
      }
    ],
    "distortionOffset": {
      "x": 1.0,
      "y": 2.0
    },
    "encoders": {
      "focus": 0.1,
      "iris": 0.2,
      "zoom": 0.3
    },
    "entrancePupilOffset": 0.123,
    "exposureFalloff": {
      "a1": 1.0,
      "a2": 2.0,
      "a3": 3.0
    },
    "fStop": 4.0,
    "pinholeFocalLength": 24.305,
    "focusDistance": 10.0,
    "projectionOffset": {
      "x": 0.1,
      "y": 0.2
    },
    "rawEncoders": {
      "focus": 1000,
      "iris": 2000,
      "zoom": 3000
    },
    "tStop": 4.1
  },
  "protocol": {
    "name": "OpenTrackIO",
    "version": [
      1,
      0,
      0
    ]
  },
  "sampleId": "urn:uuid:939fb522-68f1-40cc-a971-b7a5b7aededa",
  "sourceId": "urn:uuid:1530983c-4fbf-4cf5-b054-e6848365ca67",
  "sourceNumber": 1,
  "relatedSampleIds": [
    "urn:uuid:fb7f644e-29f9-4c8d-910e-9b26a2639863",
    "urn:uuid:3384c500-e45c-4bc2-ab67-4e18efad7152"
  ],
  "globalStage": {
    "E": 100.0,
    "N": 200.0,
    "U": 300.0,
    "lat0": 100.0,
    "lon0": 200.0,
    "h0": 300.0
  },
  "transforms": [
    {
      "translation": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "rotation": {
        "pan": 180.0,
        "tilt": 90.0,
        "roll": 45.0
      },
      "id": "Dolly"
    },
    {
      "translation": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "rotation": {
        "pan": 180.0,
        "tilt": 90.0,
        "roll": 45.0
      },
      "scale": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "id": "Crane Arm"
    },
    {
      "translation": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "rotation": {
        "pan": 180.0,
        "tilt": 90.0,
        "roll": 45.0
      },
      "scale": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "id": "Camera"
    }
  ]
}
