{
  "tracker": {
    "notes": "Example generated sample.",
    "recording": false,
    "slate": "A101_A_4",
    "status": "Optical Good"
  },
  "timing": {
    "mode": "external",
    "sampleRate": {
      "num": 24000,
      "denom": 1001
    },
    "timecode": {
      "hours": 1,
      "minutes": 2,
      "seconds": 3,
      "frames": 4,
      "frameRate": {
        "num": 24000,
        "denom": 1001
      }
    }
  },
  "lens": {
    "distortion": [
      {
        "radial": [
          1.0,
          2.0,
          3.0
        ],
        "tangential": [
          1.0,
          2.0
        ],
        "overscan": 3.1
      }
    ],
    "encoders": {
      "focus": 0.1,
      "iris": 0.2,
      "zoom": 0.3
    },
    "entrancePupilOffset": 0.123,
    "fStop": 4.0,
    "pinholeFocalLength": 24.305,
    "focusDistance": 10.0,
    "projectionOffset": {
      "x": 0.1,
      "y": 0.2
    }
  },
  "protocol": {
    "name": "OpenTrackIO",
    "version": [
      1,
      0,
      0
    ]
  },
  "sampleId": "urn:uuid:771aac95-fa2d-49a8-9fdb-1e07afb6ffa0",
  "sourceId": "urn:uuid:6dcc8625-645e-404f-8ed4-0edb18055dd1",
  "sourceNumber": 1,
  "transforms": [
    {
      "translation": {
        "x": 1.0,
        "y": 2.0,
        "z": 3.0
      },
      "rotation": {
        "pan": 180.0,
        "tilt": 90.0,
        "roll": 45.0
      },
      "id": "Camera"
    }
  ]
}