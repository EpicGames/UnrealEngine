// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkOpenTrackIO, Log, All);

/**
 * This module reads OpenTrackIO data from a UDP source.
 */
class FLiveLinkOpenTrackIOModule : public IModuleInterface
{
public:

	static FLiveLinkOpenTrackIOModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FLiveLinkOpenTrackIOModule>(TEXT("LiveLinkOpenTrackIO"));
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
