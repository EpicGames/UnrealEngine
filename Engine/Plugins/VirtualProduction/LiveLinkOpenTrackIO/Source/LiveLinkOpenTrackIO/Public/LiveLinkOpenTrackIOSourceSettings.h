// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"

#include "LiveLinkOpenTrackIOConnectionSettings.h"

#include "LiveLinkOpenTrackIOSourceSettings.generated.h"

UENUM()
enum class ELiveLinkOpenTrackIOTransformSubjects
{
	NoTransformSubjects					 UMETA(ToolTip = "Do not generate transform subjects from OpenTrackIO data."),
	EnableTransformSubjects				 UMETA(ToolTip = "Generate transform subjects and calculate final transform on the camera frame data."),
	EnableTransformSubjectsNoCameraXForm UMETA(ToolTip = "Generate transform subjects and do not calculate final transform on the camera frame data."),
};

UCLASS()
class LIVELINKOPENTRACKIO_API ULiveLinkOpenTrackIOSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()

public:

	/** Protocol being used for the connection. */
	UPROPERTY(VisibleAnywhere, Category = Transport)
	ELiveLinkOpenTrackIONetworkProtocol Protocol = ELiveLinkOpenTrackIONetworkProtocol::Multicast;

	/**
	 * The Multicast endpoint to receive packets on.
	 *
	 * The default value is 239.135.1.1:55555 per OpenTrackIO specifications.
	 */
	UPROPERTY(VisibleAnywhere, Category = Transport, meta = (
		EditCondition = "Protocol == ELiveLinkOpenTrackIONetworkProtocol::Multicast",
		EditConditionHides
	))
	FString MulticastEndpoint = TEXT("239.135.1.1:55555");

	/**
	 * The IP endpoint to listen to and send packets from.  
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 * 0.0.0.0:0 will bind to the default network adapter on Windows,
	 * and all available network adapters on other operating systems.
	 *
	 */
	UPROPERTY(VisibleAnywhere, Category=Transport)
	FString UnicastEndpoint = TEXT("0.0.0.0:0");

	/**
	 * Emit subjects for each transform included in the OpenTrackIO channel.
	 */
	UPROPERTY(EditAnywhere, Category=Subjects)
	ELiveLinkOpenTrackIOTransformSubjects SubjectsPerTransform = ELiveLinkOpenTrackIOTransformSubjects::NoTransformSubjects;

	bool ShouldExtractTransformSubjects() const
	{
		return SubjectsPerTransform != ELiveLinkOpenTrackIOTransformSubjects::NoTransformSubjects;
	}

	bool ShouldApplyXformToCamera() const
	{
		return SubjectsPerTransform != ELiveLinkOpenTrackIOTransformSubjects::EnableTransformSubjectsNoCameraXForm;
	}
};
